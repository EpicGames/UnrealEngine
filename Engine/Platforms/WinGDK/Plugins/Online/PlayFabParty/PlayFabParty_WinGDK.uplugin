{
	"FileVersion": 3,
	"bIsPluginExtension": true,
	"SupportedTargetPlatforms": [ "WinGDK" ],
	"Modules": [
		{
			"Name": "PlayFabParty",
			"Type": "Runtime",
			"LoadingPhase": "Default",
			"PlatformAllowList": [ "WinGDK" ]
		}
	]
}
