{
	"FileVersion": 3,
	"FriendlyName": "PlayFab Party",
	"Version": 1,
	"VersionName": "1.0",
	"Description": "PlayFab Party Socket Subsystem plugin",
	"Category": "Online Platform",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"EnabledByDefault": false,
	"RequiresBuildPlatform": false,
	"HasExplicitPlatforms": true,
	"ExplicitlyLoaded": true,
	"SupportedTargetPlatforms": ["Win64"],
	"Modules": [
		{
			"Name": "PlayFabParty",
			"Type": "RuntimeNoCommandlet",
			"LoadingPhase": "Default",
			"HasExplicitPlatforms": true,
			"PlatformAllowList": [ "Win64" ]
		}
	],
	"Plugins": [
		{
			"Name": "OnlineSubsystemUtils",
			"Enabled": true
		}
	]
}
