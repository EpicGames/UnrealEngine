// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class PlayFabParty : ModuleRules
{
	public PlayFabParty(ReadOnlyTargetRules Target) : base(Target)
	{
		PCHUsage = ModuleRules.PCHUsageMode.UseExplicitOrSharedPCHs;

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Sockets",
				"OnlineSubsystemUtils",
				"OnlineSubsystem",
				"GRDK",
				"GDKRuntime",
				"Json",
				"Engine",
				"CoreUObject",
				"ApplicationCore",
			}
			);

		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
			}
			);

		bool bHasPlayFabParty = false;
		if (GRDK.IsValid(Target))
		{
			if (GRDK.IsLegacyFolderStructure())
			{
				GRDK.AddLegacyExtensionDependency(Target, this, "PlayFab.Party.Cpp", "Party");
				GRDK.AddLegacyExtensionDependency(Target, this, "PlayFab.PartyXboxLive.Cpp", "PartyXboxLive");
				if (GRDK.GetGDKEdition() >= 251000)
				{
					GRDK.AddLegacyExtensionDependency(Target, this, "PlayFab.Services.C", "PlayFabCore.GDK");
				}
			}
			else
			{
				GRDK.AddDependency(Target, this, "Party", IncludeSubPath:"playfab/party");
				GRDK.AddDependency(Target, this, "PartyXboxLive", IncludeSubPath:"playfab/party");
				GRDK.AddDependency(Target, this, "PlayFabCore", IncludeSubPath:"playfab/core");
			}

			bHasPlayFabParty = true;
		}
		PublicDefinitions.Add("WITH_PLAYFAB_PARTY=" + (bHasPlayFabParty ? "1" : "0"));

		if (Target.Platform.IsInGroup(UnrealPlatformGroup.Windows))
		{
			PublicDefinitions.Add("_GAMING_DESKTOP");
		}

		//CDA duplicated from OnlineSubsytemGDK to avoid circular dependency.
		PublicDefinitions.Add("GDK_SUBSYSTEM=FName(TEXT(\"GDK\"))");

	}
}
