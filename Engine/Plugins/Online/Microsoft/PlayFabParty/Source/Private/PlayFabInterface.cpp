// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_PLAYFAB_PARTY
#include "PlayFabInterface.h"
#include "PlayFabPartyModule.h"
#include "PlayFabPartyManager.h"
#include "PlayFabPartySocketSubsystem.h"

THIRD_PARTY_INCLUDES_START
#include <Party.h>
THIRD_PARTY_INCLUDES_END


TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe>  PFManager;

TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe>& GetPFManager()
{
	if(!PFManager)
	{
		PFManager = FPlayFabPartyModule::Get().PlayFabPartySocketSubsystem->GetPlayFabPartyManager();
	}
	return PFManager;
}


bool HaveEntityIdForXuid(const uint64 Xuid) 
{ 
	if(GetPFManager())
	{
		return GetPFManager().Get()->HaveEntityIdForXuid(Xuid);
	}
	return false;
};

const char* GetEntityIdForXuid(const uint64 Xuid)
{
	if (GetPFManager())
	{
		return (const char*)GetPFManager().Get()->GetEntityIdForXuid(Xuid);
	}
	return nullptr;
}

const char* GetEntityTokenForXuid(const uint64 Xuid)
{
	if (GetPFManager())
	{
		return (const char*)GetPFManager().Get()->GetEntityTokenForXuid(Xuid);
	}
	return nullptr;
}

#endif // WITH_PLAYFAB_PARTY
