// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_PLAYFAB_PARTY
#include "PlayFabParty.h"
#include "PlayFabPartyLog.h"

THIRD_PARTY_INCLUDES_START
#include <Party_c.h>
#include <PartyImpl.h>
THIRD_PARTY_INCLUDES_END

FString GetPlayFabPartyErrorMessage(const PartyError ErrorCode)
{
	PartyString ErrorString = nullptr;

	const PartyError GetErrorResultCode = Party::PartyManager::GetErrorMessage(ErrorCode, &ErrorString);
	if (PARTY_FAILED(GetErrorResultCode))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to get error message for error code %u due to %u", ErrorCode, GetErrorResultCode);

		return FString(TEXT("Unknown Error"));
	}

	return FString(ANSI_TO_TCHAR(ErrorString));
}
#endif // WITH_PLAYFAB_PARTY
