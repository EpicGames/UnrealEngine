// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_PLAYFAB_PARTY
#include "PlayFabPartyInternetAddr.h"
#include "PlayFabPartySocketSubsystem.h"
#include "PlayFabPartyLog.h"
#include "PlayFabParty.h"
#include "Hash/CityHash.h"

#define UE_PLAYFABPARTY_ENDPOINTID_SPLIT_CHAR TEXT(':')

FPlayFabPartyInternetAddr::FPlayFabPartyInternetAddr()
{
}

FPlayFabPartyInternetAddr::FPlayFabPartyInternetAddr(const FString& NetworkIdentifier)
{
	bool bUnused = false;
	SetIp(*NetworkIdentifier, bUnused);
}

FPlayFabPartyInternetAddr::FPlayFabPartyInternetAddr(const Party::PartyEndpoint& PlayFabEndpoint)
{
	Party::PartyNetwork* Network = nullptr;

	const PartyError GetNetworkError = PlayFabEndpoint.GetNetwork(&Network);
	if (PARTY_FAILED(GetNetworkError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Unable to retrieve network from endpoint. Error=[%ls]", *GetPlayFabPartyErrorMessage(GetNetworkError));
		return;
	}
	check(Network);

	TUniquePtr<Party::PartyNetworkDescriptor> NewNetworkDescriptor = MakeUnique<Party::PartyNetworkDescriptor>();
	const PartyError GetNetworkDescriptorError = Network->GetNetworkDescriptor(NewNetworkDescriptor.Get());
	if (PARTY_FAILED(GetNetworkDescriptorError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Unable to get network descriptor from network. Error=[%ls]", *GetPlayFabPartyErrorMessage(GetNetworkDescriptorError));
		return;
	}

	uint16 UniqueIdentifier = 0;
	const PartyError GetUniqueIdError = PlayFabEndpoint.GetUniqueIdentifier(&UniqueIdentifier);
	if (PARTY_FAILED(GetUniqueIdError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Unable to get network unique identifier from endpoint. Error=[%ls]", *GetPlayFabPartyErrorMessage(GetUniqueIdError));
		return;
	}

	NetworkDescriptor = MoveTemp(NewNetworkDescriptor);
	Port = UniqueIdentifier;
}

FPlayFabPartyInternetAddr::FPlayFabPartyInternetAddr(const FPlayFabPartyInternetAddr& Other)
	: Port(Other.Port)
{
	if (Other.NetworkDescriptor.IsValid())
	{
		NetworkDescriptor = MakeUnique<Party::PartyNetworkDescriptor>(*Other.NetworkDescriptor);
	}
}

FPlayFabPartyInternetAddr& FPlayFabPartyInternetAddr::operator=(const FPlayFabPartyInternetAddr& Other)
{
	if (this != &Other)
	{
		if (Other.NetworkDescriptor.IsValid())
		{
			NetworkDescriptor = MakeUnique<Party::PartyNetworkDescriptor>(*Other.NetworkDescriptor);
		}
		else
		{
			NetworkDescriptor.Reset();
		}

		Port = Other.Port;
	}

	return *this;
}

void FPlayFabPartyInternetAddr::SetIp(uint32 InAddr)
{
	UE_LOGF(LogPlayFabParty, Warning, "Calls to FPlayFabPartyInternetAddr::SetIp are not valid");
}

void FPlayFabPartyInternetAddr::SetIp(const TCHAR* InAddr, bool& bIsValid)
{
	const int32 MaxNetworkLength = Party::c_maxSerializedNetworkDescriptorStringLength;
	const int32 SizeOfSeperator = 1;
	const int32 MaxCharsForUint16 = 5;
	const int32 SizeOfNullTerminator = 1;
	const int32 StringSize = MaxNetworkLength + SizeOfSeperator + MaxCharsForUint16 + SizeOfNullTerminator;

	ANSICHAR Buffer[StringSize];
	FCStringAnsi::Strncpy(Buffer, TCHAR_TO_ANSI(InAddr), StringSize);

	// Extract port string if we have one
	TOptional<uint16> FoundPort;
	if (ANSICHAR* PortCharPtr = FCStringAnsi::Strrchr(&Buffer[0], UE_PLAYFABPARTY_ENDPOINTID_SPLIT_CHAR))
	{
		// Extract our port
		const int32 TempPort = FCStringAnsi::Atoi(PortCharPtr);
		if (TempPort >= 0 && TempPort <= TNumericLimits<uint16>::Max())
		{
			FoundPort = static_cast<uint16>(TempPort);
		}
		else
		{
			UE_LOGF(LogPlayFabParty, Warning, "SetIp failed due to invalid port value. Port=[%d]", TempPort);
			bIsValid = false;
			return;
		}

		// Change the split char into a null so PlayFab doesn't read past it
		*PortCharPtr = '\0';
	}

	// Undo the replacements we did in ToString: Replace any dashes (-) with pluses (+) and underscores (_) with forward slashes (/)
	for (int32 Index = 0; Index < sizeof(Buffer); ++Index)
	{
		ANSICHAR& Char = Buffer[Index];
		if (Char == '\0')
		{
			break;
		}

		if (Char == '-')
		{
			Char = '+';
		}
		else if (Char == '_')
		{
			Char = '/';
		}
	}

	// Read network descriptor info
	TUniquePtr<Party::PartyNetworkDescriptor> NewNetworkDescriptor = MakeUnique<Party::PartyNetworkDescriptor>();
	const PartyError ResultCode = Party::PartyManager::DeserializeNetworkDescriptor(&Buffer[0], NewNetworkDescriptor.Get());
	if (PARTY_FAILED(ResultCode))
	{
		UE_LOGF(LogPlayFabParty, Warning, "SetIp failed to deserialize PlayFab network descriptor: %ls", *GetPlayFabPartyErrorMessage(ResultCode));
		bIsValid = false;
		return;
	}

	// Save if we were valid
	NetworkDescriptor = MoveTemp(NewNetworkDescriptor);
	Port = FoundPort;
	bIsValid = true;
}

void FPlayFabPartyInternetAddr::GetIp(uint32& OutAddr) const
{
	UE_LOGF(LogPlayFabParty, Warning, "Calls to FPlayFabPartyInternetAddr::GetIp are not valid");
}

void FPlayFabPartyInternetAddr::SetPort(int32 InPort)
{
	if (InPort >= 0 && InPort <= TNumericLimits<uint16>::Max())
	{
		Port = static_cast<uint16>(InPort);
	}
	else
	{
		Port.Reset();
	}
}

int32 FPlayFabPartyInternetAddr::GetPort() const
{
	return Port.IsSet() ? Port.GetValue() : -1;
}

void FPlayFabPartyInternetAddr::SetRawIp(const TArray<uint8>& RawAddr)
{
	const auto ConvertedString = StringCast<TCHAR>(reinterpret_cast<const ANSICHAR*>(RawAddr.GetData()));

	bool bUnused = true;
	SetIp(ConvertedString.Get(), bUnused);
}

TArray<uint8> FPlayFabPartyInternetAddr::GetRawIp() const
{
	const FString AddrAsString = ToString(true);
	const auto ConvertedString = StringCast<ANSICHAR>(*AddrAsString);

	TArray<uint8> Result;
	Result.Append(reinterpret_cast<const uint8*>(ConvertedString.Get()), ConvertedString.Length());
	return Result;
}

void FPlayFabPartyInternetAddr::SetAnyAddress()
{
	// PlayFab Party does not support this, but clear any previously set address
	Clear();

	// We don't log/warn here, because this function is called very frequently, whether or not the socket subsystem supports it
}

void FPlayFabPartyInternetAddr::SetBroadcastAddress()
{
	// PlayFab Party does not support this, but clear any previously set address
	Clear();

	// We don't log/warn here, because this function is called very frequently, whether or not the socket subsystem supports it
}

void FPlayFabPartyInternetAddr::SetLoopbackAddress()
{
	// PlayFab Party does not support this, but clear any previously set address
	Clear();

	// We don't log/warn here, because this function is called very frequently, whether or not the socket subsystem supports it
}

FString FPlayFabPartyInternetAddr::ToString(bool bAppendPort) const
{
	FString Result;

	if (!NetworkDescriptor.IsValid())
	{
		return Result;
	}

	char Buffer[Party::c_maxSerializedNetworkDescriptorStringLength + 1];
	const PartyError ResultCode = Party::PartyManager::SerializeNetworkDescriptor(NetworkDescriptor.Get(), &Buffer[0]);
	if (PARTY_FAILED(ResultCode))
	{
		UE_LOGF(LogPlayFabParty, Warning, "FPlayFabPartyInternetAddr::ToString failed to serialize PlayFab network descriptor: %ls", *GetPlayFabPartyErrorMessage(ResultCode));
		return Result;
	}

	// The serialized network descriptor is base64, but we don't want + or / characters in our string
	// So to make it more friendly, replace any pluses (+) with dashes (-) and forward slashes (/) with underscores (_)
	for (int32 Index = 0; Index < sizeof(Buffer); ++Index)
	{
		ANSICHAR& Char = Buffer[Index];
		if (Char == '\0')
		{
			break;
		}

		if (Char == '+')
		{
			Char = '-';
		}
		else if (Char == '/')
		{
			Char = '_';
		}
	}

	const bool bShouldAppendPort = (bAppendPort && Port.IsSet());
	const int32 MaxNetworkLength = Party::c_maxSerializedNetworkDescriptorStringLength;
	const int32 SizeOfSeperator = 1;
	const int32 MaxCharsForUint16 = 5;

	const int32 StringSize = MaxNetworkLength + (bShouldAppendPort ? (SizeOfSeperator + MaxCharsForUint16) : 0);

	Result.Reserve(StringSize);
	Result.Append(&Buffer[0]);

	if (bShouldAppendPort)
	{
		Result.AppendChar(UE_PLAYFABPARTY_ENDPOINTID_SPLIT_CHAR);
		Result.AppendInt(Port.GetValue());
	}

	return Result;
}

uint32 FPlayFabPartyInternetAddr::GetTypeHash() const
{
	Party::PartyNetworkDescriptor EmptyDescriptor = {};
	const Party::PartyNetworkDescriptor* HashObject = NetworkDescriptor.IsValid() ? NetworkDescriptor.Get() : &EmptyDescriptor;

	const uint32 NetworkIdHash = CityHash32(static_cast<const char*>(&HashObject->networkIdentifier[0]), sizeof(HashObject->networkIdentifier));
	const uint32 RegionNameHash = CityHash32(static_cast<const char*>(&HashObject->regionName[0]), sizeof(HashObject->regionName));
	const uint32 ConnectionInfoHash = CityHash32(reinterpret_cast<const char*>(&HashObject->opaqueConnectionInformation[0]), sizeof(HashObject->opaqueConnectionInformation));
	const uint32 PortHash = Port.IsSet() ? Port.GetValue() : TNumericLimits<uint32>::Max();

	return HashCombine(HashCombine(HashCombine(NetworkIdHash, RegionNameHash), ConnectionInfoHash), PortHash);
}

bool FPlayFabPartyInternetAddr::IsValid() const
{
	return NetworkDescriptor.IsValid();
}

TSharedRef<FInternetAddr> FPlayFabPartyInternetAddr::Clone() const
{
	return MakeShared<FPlayFabPartyInternetAddr>(*this);
}

FName FPlayFabPartyInternetAddr::GetProtocolType() const
{
	return FName(PLAYFABPARTY_SOCKETSUBSYSTEM);
}

const Party::PartyNetworkDescriptor* FPlayFabPartyInternetAddr::GetNetworkDescriptor() const
{
	return NetworkDescriptor.Get();
}

void FPlayFabPartyInternetAddr::SetNetworkDescriptor(const Party::PartyNetworkDescriptor& NewNetworkDescriptor)
{
	NetworkDescriptor = MakeUnique<Party::PartyNetworkDescriptor>(NewNetworkDescriptor);
}

void FPlayFabPartyInternetAddr::SetNetworkDescriptor(TUniquePtr<Party::PartyNetworkDescriptor>&& NewNetworkDescriptor)
{
	NetworkDescriptor = MoveTemp(NewNetworkDescriptor);
}

void FPlayFabPartyInternetAddr::Clear()
{
	NetworkDescriptor.Reset();
	Port.Reset();
}

#undef UE_PLAYFABPARTY_ENDPOINTID_SPLIT_CHAR
#endif // WITH_PLAYFAB_PARTY
