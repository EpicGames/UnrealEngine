// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPAddress.h"
#include "Misc/Optional.h"

#if WITH_PLAYFAB_PARTY
namespace Party
{
	struct PartyNetworkDescriptor;
	class PartyEndpoint;
}

class FPlayFabPartyInternetAddr
	: public FInternetAddr
{
public:
	FPlayFabPartyInternetAddr();
	explicit FPlayFabPartyInternetAddr(const FString& SerializedNetworkDescriptor);
	explicit FPlayFabPartyInternetAddr(const Party::PartyEndpoint& PlayFabEndpoint);

	FPlayFabPartyInternetAddr(const FPlayFabPartyInternetAddr& Other);
	FPlayFabPartyInternetAddr(FPlayFabPartyInternetAddr&& Other) = default;
	FPlayFabPartyInternetAddr& operator=(const FPlayFabPartyInternetAddr& Other);
	FPlayFabPartyInternetAddr& operator=(FPlayFabPartyInternetAddr&& Other) = default;
	virtual ~FPlayFabPartyInternetAddr() = default;

	//~ Begin FInternetAddr Interface
	virtual void SetIp(uint32 InAddr) override;
	virtual void SetIp(const TCHAR* InAddr, bool& bIsValid) override;
	virtual void GetIp(uint32& OutAddr) const override;
	virtual void SetPort(int32 InPort) override;
	virtual int32 GetPort() const override;
	virtual void SetRawIp(const TArray<uint8>& RawAddr) override;
	virtual TArray<uint8> GetRawIp() const override;
	virtual void SetAnyAddress() override;
	virtual void SetBroadcastAddress() override;
	virtual void SetLoopbackAddress() override;
	virtual FString ToString(bool bAppendPort) const override;
	virtual uint32 GetTypeHash() const override;
	virtual bool IsValid() const override;
	virtual TSharedRef<FInternetAddr> Clone() const override;
	virtual FName GetProtocolType() const override;
	//~ End FInternetAddr Interface

	/** Get the Network Descriptor for this internet address */
	const Party::PartyNetworkDescriptor* GetNetworkDescriptor() const;
	/** Set the Network Descriptor to a new value */
	void SetNetworkDescriptor(const Party::PartyNetworkDescriptor& NewNetworkDescriptor);
	/** Set the Network Descriptor to a new value */
	void SetNetworkDescriptor(TUniquePtr<Party::PartyNetworkDescriptor>&& NewNetworkDescriptor);
	/** Clears all set state of this object */
	void Clear();

protected:
	/** The unique address of this network */
	TUniquePtr<Party::PartyNetworkDescriptor> NetworkDescriptor;

	/** The unique identifier for a specific endpoint on this network (or unset if we're not talking about a specific user) */
	TOptional<uint16> Port;
};
#endif // WITH_PLAYFAB_PARTY
