// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_PLAYFAB_PARTY
#include "PlayFabPartyLive.h"

THIRD_PARTY_INCLUDES_START
#include <Party_c.h>
#include <PartyXboxLive_c.h>
#include <PartyXboxLiveImpl.h>
THIRD_PARTY_INCLUDES_END

#endif // WITH_PLAYFAB_PARTY
