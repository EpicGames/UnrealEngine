// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_PLAYFAB_PARTY

#include "CoreMinimal.h"

THIRD_PARTY_INCLUDES_START
#include <Party.h>
#include <PartyXboxLive.h>
THIRD_PARTY_INCLUDES_END

#endif // WITH_PLAYFAB_PARTY
