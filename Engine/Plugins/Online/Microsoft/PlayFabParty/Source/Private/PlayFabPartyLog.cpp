// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayFabPartyLog.h"

DEFINE_LOG_CATEGORY(LogPlayFabParty);
