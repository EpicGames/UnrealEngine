// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_PLAYFAB_PARTY
#include "PlayFabPartyManager.h"
#include "PlayFabParty.h"
#include "PlayFabPartyLive.h"
#include "PlayFabPartyLog.h"
#include "PlayFabPartySocket.h"
#include "PlayFabPartySocketSubsystem.h"

#include "Containers/Ticker.h"
#include "Containers/StringView.h"
#include "Containers/StringConv.h"
#include "GDKTaskQueueHelpers.h"
#include "GDKRuntimeModule.h"
#include "GDKThreadCheck.h"
#include "HAL/PlatformTime.h"
#include "HAL/PlatformMisc.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/CoreDelegates.h"
#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"
#include "Online/OnlineSessionNames.h"
#include "Interfaces/OnlineSessionInterface.h"
#include "Policies/CondensedJsonPrintPolicy.h"
#include "Serialization/JsonWriter.h"
#include "Templates/UnrealTemplate.h"
#include <vector>



using FJsonWriter = TSharedRef<TJsonWriter<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>>;
using FJsonWriterFactory = TJsonWriterFactory<TCHAR, TCondensedJsonPrintPolicy<TCHAR>>;

namespace
{
	bool IsPlayFabPartyAppIdValid(const FStringView AppId)
	{
		const FStringView TrimmedAppId = AppId.TrimStartAndEnd();
		if (TrimmedAppId.IsEmpty())
		{
			return false;
		}

		for (int32 Index = 0; Index < TrimmedAppId.Len(); ++Index)
		{
			const TCHAR Char = TrimmedAppId[Index];
			if (!FChar::IsHexDigit(Char))
			{
				return false;
			}
		}

		return true;
	}
}

FPlayFabPartyManager::~FPlayFabPartyManager()
{
	UnbindSystemDelegates();

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
	if (bIsPlayFabPartyInit || bIsPlayFabPartyXboxInit || bIsPlayFabCoreInit)
#else
	if (bIsPlayFabPartyInit || bIsPlayFabPartyXboxInit)
#endif
	{
		ShutdownPlayFabParty(true);
	}
}

FPlayFabPartyEntityId::FPlayFabPartyEntityId(PartyString EntityId)
{
	const int32 StringLength = FCStringAnsi::Strlen(EntityId);

	// Copy the string and its null terminator
	Data.Append(EntityId, StringLength + 1);
}

PartyString FPlayFabPartyEntityId::GetEntityId() const
{
	return Data.GetData();
}

FPlayFabPartyEntityToken::FPlayFabPartyEntityToken(PartyString EntityToken)
{
	const int32 StringLength = FCStringAnsi::Strlen(EntityToken);

	// Copy the string and its null terminator
	Data.Append(EntityToken, StringLength + 1);
}

PartyString FPlayFabPartyEntityToken::GetEntityToken() const
{
	return Data.GetData();
}

/*static*/
TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> FPlayFabPartyManager::CreateManager(FPlayFabPartySocketSubsystem& OwningSocketSubsystem, const FString& AppId, FString& OutError)
{
	TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> Manager;

	if (IsPlayFabPartyAppIdValid(AppId))
	{
		Manager = MakeShareable(new FPlayFabPartyManager(OwningSocketSubsystem, AppId));
		if (!Manager->BindSystemDelegates())
		{
			UE_LOGF(LogPlayFabParty, Error, "Failed to initialize required system bindings");
			Manager.Reset();

			OutError = TEXT("Failed to bind to system delegates");
		}
	}
	else
	{
		OutError = TEXT("PlayFab Party AppId was invalid");
	}

	return Manager;
}

/*static*/
TOptional<FString> FPlayFabPartyManager::GetAppId()
{
	TOptional<FString> Result;

	FString TempPlayFabPartyAppId;
	if (GConfig->GetString(TEXT("PlayFab"), TEXT("AppId"), TempPlayFabPartyAppId, GEngineIni))
	{
		if (IsPlayFabPartyAppIdValid(TempPlayFabPartyAppId))
		{
			Result.Emplace(MoveTemp(TempPlayFabPartyAppId));
		}
	}

	return Result;
}

bool FPlayFabPartyManager::IsInitialized() const
{
#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
	return bIsPlayFabCoreInit && bIsPlayFabPartyInit && bIsPlayFabPartyXboxInit;
#else
	return bIsPlayFabPartyInit && bIsPlayFabPartyXboxInit; 
#endif
}

bool FPlayFabPartyManager::IsReady() const
{
	return IsInitialized() && LocalUserLoginState == EPlayFabPartyLoginState::LoggedIn;
}

bool FPlayFabPartyManager::BindSystemDelegates()
{
	bIsNetworkReady = IGDKRuntimeModule::Get().IsNetworkInitialized();

	// Bind app suspend/resume handlers to shutdown playfab
	AppSuspendingHandle = FCoreDelegates::ApplicationWillEnterBackgroundDelegate.AddRaw(this, &FPlayFabPartyManager::HandleAppSuspended);
	AppResumingHandle = FCoreDelegates::ApplicationHasEnteredForegroundDelegate.AddRaw(this, &FPlayFabPartyManager::HandleAppResume);
	NetworkInitializationChangedHandle = FCoreDelegates::ApplicationNetworkInitializationChanged.AddRaw(this, &FPlayFabPartyManager::HandleNetworkInitializationChanged);

	// Bind OSS
	IOnlineSubsystem* OnlineSubsystemGDK = IOnlineSubsystem::Get(GDK_SUBSYSTEM);
	if (!OnlineSubsystemGDK)
	{
		UnbindSystemDelegates();
		return false;
	}

	IOnlineSessionPtr OnlineSessionInterface = OnlineSubsystemGDK->GetSessionInterface();
	if (!OnlineSessionInterface.IsValid())
	{
		UnbindSystemDelegates();
		return false;
	}

	QOSDataRequestedHandle = OnlineSessionInterface->AddOnQosDataRequestedDelegate_Handle(FOnQosDataRequestedDelegate::CreateThreadSafeSP(this, &FPlayFabPartyManager::HandleQOSDataRequested));

	UpdatePlayFabPartyStatus();
	return true;
}

void FPlayFabPartyManager::UnbindSystemDelegates()
{
	if (IOnlineSubsystem::IsLoaded(GDK_SUBSYSTEM))
	{
		if (IOnlineSubsystem* OnlineSubsystemGDK = IOnlineSubsystem::Get(GDK_SUBSYSTEM))
		{
			if (IOnlineSessionPtr OnlineSessionInterface = OnlineSubsystemGDK->GetSessionInterface())
			{
				if (QOSDataRequestedHandle.IsValid())
				{
					OnlineSessionInterface->ClearOnQosDataRequestedDelegate_Handle(QOSDataRequestedHandle);
					QOSDataRequestedHandle.Reset();
				}
			}
		}
	}

	if (NetworkInitializationChangedHandle.IsValid())
	{
		FCoreDelegates::ApplicationNetworkInitializationChanged.Remove(NetworkInitializationChangedHandle);
		NetworkInitializationChangedHandle.Reset();
	}

	if (AppResumingHandle.IsValid())
	{
		FCoreDelegates::ApplicationHasEnteredForegroundDelegate.Remove(AppResumingHandle);
		AppResumingHandle.Reset();
	}

	if (AppSuspendingHandle.IsValid())
	{
		FCoreDelegates::ApplicationWillEnterBackgroundDelegate.Remove(AppSuspendingHandle);
		AppSuspendingHandle.Reset();
	}
}

Party::PartyLocalUser* FPlayFabPartyManager::GetPartyLocalUser()
{
	return PartyLocalUser;
}

const Party::PartyLocalUser* FPlayFabPartyManager::GetPartyLocalUser() const
{
	return PartyLocalUser;
}

Party::PartyXblLocalChatUser* FPlayFabPartyManager::GetPartyXboxLocalChatUser()
{
	return PartyXboxLocalChatUser;
}

const Party::PartyXblLocalChatUser* FPlayFabPartyManager::GetPartyXboxLocalChatUser() const
{
	return PartyXboxLocalChatUser;
}

const TOptional<FString>& FPlayFabPartyManager::GetQOSString() const
{
	return QOSData;
}

TUniquePtr<Party::PartyNetworkDescriptor> FPlayFabPartyManager::CreateNetwork(const FString& InitialInvitationString, const uint32 MaxPlayers, void* CallbackContext) const
{
	// Ensure PlayFab Party is in correct state
	if (!IsReady())
	{
		UE_LOGF(LogPlayFabParty, Error, "PlayFabParty networks cannot be created until we are successfully logged into PlayFabParty.");
		return nullptr;
	}

	// Validate input
	if (MaxPlayers > Party::c_maxNetworkConfigurationMaxDeviceCount)
	{
		return nullptr;
	}

	TUniquePtr<Party::PartyNetworkDescriptor> Result = MakeUnique<Party::PartyNetworkDescriptor>();

	// Create network
	Party::PartyNetworkConfiguration NetworkConfig = {0};
	NetworkConfig.maxUserCount = MaxPlayers;
	NetworkConfig.maxDeviceCount = MaxPlayers;
	NetworkConfig.maxUsersPerDeviceCount = Party::c_maxLocalUsersPerDeviceCount;
	NetworkConfig.maxDevicesPerUserCount = 1;
	NetworkConfig.maxEndpointsPerDeviceCount = 1;

	const uint32 RegionCount = 0;
	const Party::PartyRegion* RegionList = nullptr;

	Party::PartyInvitationConfiguration InitialInvitationConfig = {0};
	const auto ConvertedInitialInvitationString = StringCast<ANSICHAR>(*InitialInvitationString);
	InitialInvitationConfig.identifier = ConvertedInitialInvitationString.Get();
	InitialInvitationConfig.revocability = Party::PartyInvitationRevocability::Anyone;
	InitialInvitationConfig.entityIdCount = 0;
	InitialInvitationConfig.entityIds = nullptr;

	void* AsyncIdentifer = CallbackContext;
	char* InitialInviteBuffer = nullptr;

	PartyError CreateNetworkError = Party::PartyManager::GetSingleton().CreateNewNetwork(PartyLocalUser, &NetworkConfig, RegionCount, RegionList, &InitialInvitationConfig, AsyncIdentifer, Result.Get(), InitialInviteBuffer);
	if (PARTY_FAILED(CreateNetworkError))
	{
		UE_LOGF(LogPlayFabParty, Error, "Failed to create PlayFabParty due to error. ErrorCode=[%u] Error=[%ls]", CreateNetworkError, *GetPlayFabPartyErrorMessage(CreateNetworkError));
		return nullptr;
	}

	return Result;
}

bool FPlayFabPartyManager::HaveEntityIdForXuid(const uint64 Xuid)
{
	return XboxUserIdToPlayFabEntityIdMap.Find(Xuid) != nullptr;
}

PartyString FPlayFabPartyManager::GetEntityIdForXuid(const uint64 Xuid)
{
	return XboxUserIdToPlayFabEntityIdMap.FindChecked(Xuid).Get().GetEntityId();
}

PartyString FPlayFabPartyManager::GetEntityTokenForXuid(const uint64 Xuid)
{
	if (auto Token = XboxUserIdToPlayFabEntityTokenMap.Find(Xuid))
	{
		return Token->Get().GetEntityToken();
	}
	return nullptr;
}

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
PFEntityHandle FPlayFabPartyManager::GetEntityHandleForXuid(const uint64 Xuid)
{
	if (PFEntityHandle* EntityHandle = XboxUserIdToPlayFabEntityHandleMap.Find(Xuid))
	{
		return *EntityHandle;
	}

	return nullptr;
}
#endif


FPlayFabPartyManager::FPlayFabPartyManager(FPlayFabPartySocketSubsystem& InSocketSubsystem, const FString& InAppId)
	: SocketSubsystem(InSocketSubsystem)
	, AppId(InAppId)
{
	GConfig->GetDouble(TEXT("PlayFab"), TEXT("LoginFailureDelaySeconds"), LoginFailureDelaySeconds, GEngineIni);
	if (!GConfig->GetString(TEXT("PlayFab"), TEXT("PlayFabURL"), PlayFabURL, GEngineIni))
	{
		auto ConvertedAppId = StringCast<ANSICHAR>(*AppId);
		PlayFabURL = FString::Printf(TEXT("https://%hs.playfabapi.com"), ConvertedAppId.Get());
	}
	
#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE // Token refresh is handled automatically after 251000
	GConfig->GetDouble(TEXT("PlayFab"), TEXT("LoginRefreshDelaySeconds"), LoginRefreshDelaySeconds, GEngineIni);
#endif
}

void FPlayFabPartyManager::UpdatePlayFabPartyStatus()
{
	check(IsInGameThread());

	if (bIsPlayFabPartyInit)
	{
		if (!bIsNetworkReady || bIsAppSuspending)
		{
			ShutdownPlayFabParty();
		}
	}
	else
	{
		if (bIsNetworkReady && !bIsAppSuspending)
		{
			InitPlayFabParty();
		}
	}
}

void FPlayFabPartyManager::InitPlayFabParty()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (startup only) PartyManager & PartyXblManager Initialize call many functions which are not safe to call on time-sensitive threads
	UE_LOGF(LogPlayFabParty, Log, "Initializing PlayFabParty");

	check(!bIsPlayFabPartyInit);
	check(!bIsPlayFabPartyXboxInit);

	// This is needed in order to pass task context to the queue, it can't be captured by the lambda since only non-capturing lambdas can implicitly convert to a raw function pointer.
	TWeakPtr<FPlayFabPartyManager>* WeakThisPtr = new TWeakPtr<FPlayFabPartyManager>();
	*WeakThisPtr = this->AsWeak();

	HRESULT InitPFPartySubmitTaskResult = XTaskQueueSubmitCallback(
		FGDKAsyncTaskQueue::GetGenericSerialQueue(),
		XTaskQueuePort::Work,
		WeakThisPtr,
		[](void* context, bool cancel)
		{

			TWeakPtr<FPlayFabPartyManager>* WeakPFPartyManagerPtr = static_cast<TWeakPtr<FPlayFabPartyManager>*>(context);
			TSharedPtr<FPlayFabPartyManager> SharedPFPartyManagerPtr = WeakPFPartyManagerPtr->Pin();
			if (!SharedPFPartyManagerPtr.IsValid())
			{
				delete WeakPFPartyManagerPtr;
				UE_LOGF(LogPlayFabParty, Error, "[InitPlayFabParty PFInitialize] PlayFabManager was destroyed before the async block was executed");
				return;
			}

			if (SharedPFPartyManagerPtr->IsInitialized())
			{
				delete WeakPFPartyManagerPtr;
				UE_LOGF(LogPlayFabParty, Log, "[InitPlayFabParty PFInitialize] PlayFabManager was already initialized");
				return;
			}

			auto ConvertedAppId = StringCast<ANSICHAR>(*SharedPFPartyManagerPtr->AppId);
			
#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE

			HRESULT PFCoreInitResult = PFInitialize(nullptr);
			if (FAILED(PFCoreInitResult))
			{
				delete WeakPFPartyManagerPtr;
				UE_LOGF(LogPlayFabParty, Error, "Failed to initialize PlayFabCore. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreInitResult, *GetPlayFabPartyErrorMessage(PFCoreInitResult));
				return;
			}

			PFServiceConfigHandle LocalServiceConfigHandle = { nullptr };
			HRESULT PFCoreConfigHandleResult = PFServiceConfigCreateHandle(TCHAR_TO_ANSI(*SharedPFPartyManagerPtr->PlayFabURL), ConvertedAppId.Get(), &LocalServiceConfigHandle);
			if (FAILED(PFCoreConfigHandleResult))
			{
				UE_LOGF(LogPlayFabParty, Error, "Failed to initialize PlayFabCore config handle. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreConfigHandleResult, *GetPlayFabPartyErrorMessage(PFCoreConfigHandleResult));

				SharedPFPartyManagerPtr->bIsPlayFabCoreInit = false;
			}
			else
			{
				SharedPFPartyManagerPtr->PFCoreServiceHandle.store(LocalServiceConfigHandle);
				SharedPFPartyManagerPtr->bIsPlayFabCoreInit = true;
			}

			const Party::PartyInitializationConfiguration PartyInitConfig =
			{
				.titleId = ConvertedAppId.Get(),
			};

			const PartyError InitializePlayFabErrorCode = Party::PartyManager::GetSingleton().Initialize(&PartyInitConfig);
#else

			const PartyError InitializePlayFabErrorCode = Party::PartyManager::GetSingleton().Initialize(ConvertedAppId.Get());
#endif

			
			if (PARTY_FAILED(InitializePlayFabErrorCode))
			{
				UE_LOGF(LogPlayFabParty, Warning, "Failed to initialize PlayFabParty. ErrorCode=[%u] Error=[%ls]", InitializePlayFabErrorCode, *GetPlayFabPartyErrorMessage(InitializePlayFabErrorCode));
				// We don't try to try again because config is probably invalid if this fails
				SharedPFPartyManagerPtr->bIsPlayFabPartyInit = false;
			}
			else
			{
				SharedPFPartyManagerPtr->bIsPlayFabPartyInit = true;
			}

			const PartyError InitializePlayFabXboxErrorCode = Party::PartyXblManager::GetSingleton().Initialize(ConvertedAppId.Get());
			if (PARTY_FAILED(InitializePlayFabXboxErrorCode))
			{
				UE_LOGF(LogPlayFabParty, Warning, "Failed to initialize PlayFabParty Xbox. ErrorCode=[%u] Error=[%ls]", InitializePlayFabXboxErrorCode, *GetPlayFabPartyErrorMessage(InitializePlayFabXboxErrorCode));
				// We don't try to try again because config is probably invalid if this fails
				SharedPFPartyManagerPtr->bIsPlayFabPartyXboxInit = false;
			}
			else
			{
				SharedPFPartyManagerPtr->bIsPlayFabPartyXboxInit = true;
			}

			HRESULT InitPFPartySubmitTaskResult = XTaskQueueSubmitCallback(
				FGDKAsyncTaskQueue::GetGenericSerialQueue(),
				XTaskQueuePort::Completion,
				WeakPFPartyManagerPtr,
				[](void* context, bool cancel)
					{

					TWeakPtr<FPlayFabPartyManager>* WeakPFPartyManagerPtr = static_cast<TWeakPtr<FPlayFabPartyManager>*>(context);
					TSharedPtr<FPlayFabPartyManager> SharedPFPartyManagerPtr = WeakPFPartyManagerPtr->Pin();
						if (!SharedPFPartyManagerPtr.IsValid())
						{
							delete WeakPFPartyManagerPtr;
							UE_LOGF(LogPlayFabParty, Error, "[InitPlayFabParty PFInitialize 2] PlayFabManager was destroyed before the async block was executed");
							return;
						}

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
						if (!SharedPFPartyManagerPtr->bIsPlayFabPartyInit 
							|| !SharedPFPartyManagerPtr->bIsPlayFabPartyXboxInit 
							|| !SharedPFPartyManagerPtr->bIsPlayFabCoreInit)
#else
						if (!SharedPFPartyManagerPtr->bIsPlayFabPartyInit 
							|| !SharedPFPartyManagerPtr->bIsPlayFabPartyXboxInit)
#endif

						{
							delete WeakPFPartyManagerPtr;
							SharedPFPartyManagerPtr->ShutdownPlayFabParty();
							return;
						}

						SharedPFPartyManagerPtr->TickHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateRaw(SharedPFPartyManagerPtr.Get(), &FPlayFabPartyManager::Tick), 0.0f);
						delete WeakPFPartyManagerPtr;
					});

			if (!SUCCEEDED(InitPFPartySubmitTaskResult))
			{
				delete WeakPFPartyManagerPtr;
				UE_LOGF(LogPlayFabParty, Error, "PlayFabParty failed to upload Init task to process queue 2. ErrorCode=[%0.8x] ErrorMessage=[%ls]", InitPFPartySubmitTaskResult, *GetPlayFabPartyErrorMessage(InitPFPartySubmitTaskResult));
			}
		});

	if (!SUCCEEDED(InitPFPartySubmitTaskResult))
	{
		delete WeakThisPtr;
		UE_LOGF(LogPlayFabParty, Error, "PlayFabParty failed to upload Init task to process queue. ErrorCode=[%0.8x] ErrorMessage=[%ls]", InitPFPartySubmitTaskResult, *GetPlayFabPartyErrorMessage(InitPFPartySubmitTaskResult));
		
	}
}

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
void FPlayFabPartyManager::ShutdownPlayFabCore(bool bForceRunSync)
{
	if (!ensureMsgf(bIsPlayFabCoreInit, TEXT("PlayFabCore wasn't initialized")) )
	{
		return;
	}

	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) PFUninitializeAsync cleanup calls many functions which are not safe to call on time-sensitive threads
	for (TTuple<uint64, PFEntityHandle>& IdToHandle : XboxUserIdToPlayFabEntityHandleMap)
	{
		if (IdToHandle.Value)
		{
			PFEntityCloseHandle(IdToHandle.Value);
			IdToHandle.Value = nullptr;
		}
		else
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to cleanup PlayFabCore EntityHandle, invalid handle");
		}
	}
	XboxUserIdToPlayFabEntityHandleMap.Empty();

	UE_LOGF(LogPlayFabParty, Log, "Shutting down PlayFabCore");
	if (PFCoreServiceHandle)
	{
		PFServiceConfigCloseHandle(PFCoreServiceHandle);
		PFCoreServiceHandle = nullptr;
		UE_LOGF(LogPlayFabParty, Log, "PlayFabCore service config handle cleaned");
	}
	else
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to cleanup PlayFabCore service config, handle was null");
	}

	

	if (bForceRunSync)
	{
		XAsyncBlock PFCoreShutdownAsyncBlock{};

		HRESULT PFCoreShutdownResult = PFUninitializeAsync(&PFCoreShutdownAsyncBlock);
		if (!SUCCEEDED(PFCoreShutdownResult))
		{
			UE_LOGF(LogPlayFabParty, Error, "PlayFabCore uninitialization sync call failed. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreShutdownResult, *GetPlayFabPartyErrorMessage(PFCoreShutdownResult));
			return;
		}

		PFCoreShutdownResult = XAsyncGetStatus(&PFCoreShutdownAsyncBlock, true);
		if (SUCCEEDED(PFCoreShutdownResult))
		{
			UE_LOGF(LogPlayFabParty, Log, "PlayFabCore uninitialized successfully");
		}
		else
		{
			UE_LOGF(LogPlayFabParty, Error, "PlayFabCore uninitialization failed. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreShutdownResult, *GetPlayFabPartyErrorMessage(PFCoreShutdownResult));
		}

		return;
	}

	HRESULT PFCoreShutdownResult = AsyncGDKTask(
		[weakThis = this->AsWeak()](XAsyncBlock* PFCoreShutdownAsyncBlock) -> HRESULT
		{
			if (TSharedPtr<FPlayFabPartyManager> PFPartyManagerPtr = weakThis.Pin())
			{
				HRESULT PFCoreShutdownResult = PFUninitializeAsync(PFCoreShutdownAsyncBlock);
				if (!SUCCEEDED(PFCoreShutdownResult))
				{
					UE_LOGF(LogPlayFabParty, Warning, "[ShutdownPlayFabCore StartGDKTask] PlayFabCore uninitialization async call failed. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreShutdownResult, *GetPlayFabPartyErrorMessage(PFCoreShutdownResult));
				}

				return PFCoreShutdownResult;
			}

			UE_LOGF(LogPlayFabParty, Warning, "[ShutdownPlayFabCore StartGDKTask] PlayFabManager was destroyed before the async block was executed");
			return E_ABORT;
		},
		[weakThis = this->AsWeak()](XAsyncBlock* PFCoreLoginAsyncBlock)
		{
			TSharedPtr<FPlayFabPartyManager> PFPartyManagerPtr = weakThis.Pin();
			if (!PFPartyManagerPtr)
			{
				UE_LOGF(LogPlayFabParty, Warning, "[ShutdownPlayFabCore GDKTaskCallback] PlayFabManager was destroyed before the async block was executed");
				return;
			}

			UE_LOGF(LogPlayFabParty, Log, "PlayFabCore uninitialized successfully");
		}
	, FGDKAsyncTaskQueue::GetGenericSerialQueue());

	if (!SUCCEEDED(PFCoreShutdownResult))
	{
		UE_LOGF(LogPlayFabParty, Warning, "PlayFabCore uninitialization async call failed. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreShutdownResult, *GetPlayFabPartyErrorMessage(PFCoreShutdownResult));
	}

}
#endif

void FPlayFabPartyManager::CleanupPlayFabPartySingleton()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) PartyManager cleanup calls many functions which are not safe to call on time-sensitive threads

	// Cleanup PlayFabParty
	PartyError CleanupErrorCode = Party::PartyManager::GetSingleton().Cleanup();
	if (PARTY_FAILED(CleanupErrorCode))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to cleanup PlayFabParty. ErrorCode=[%u] Error=[%ls]", CleanupErrorCode, *GetPlayFabPartyErrorMessage(CleanupErrorCode));
		// Still unbind and pretend we shutdown successfully if we fail
	}

	UE_LOGF(LogPlayFabParty, Log, "PlayFabParty successfully cleanup.");
}

void FPlayFabPartyManager::CleanupPlayFabPartyXboxSingleton()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); // (shutdown only) PartyXblManager cleanup calls many functions which are not safe to call on time-sensitive threads

	// Cleanup PlayFabParty Xbox
	const PartyError CleanupXboxErrorCode = Party::PartyXblManager::GetSingleton().Cleanup();
	if (PARTY_FAILED(CleanupXboxErrorCode))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to cleanup PlayFabParty Xbox. ErrorCode=[%u] Error=[%ls]", CleanupXboxErrorCode, *GetPlayFabPartyErrorMessage(CleanupXboxErrorCode));
		// Still unbind and pretend we shutdown successfully if we fail
	}

	UE_LOGF(LogPlayFabParty, Log, "PlayFabParty Xbox successfully cleanup.");
}

void FPlayFabPartyManager::ShutdownPlayFabParty(bool bForceRunSync)
{
	UE_LOGF(LogPlayFabParty, Log, "Shutting down PlayFabParty");

	if (bForceRunSync)
	{
#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
		check(bIsPlayFabPartyInit || bIsPlayFabPartyXboxInit || bIsPlayFabCoreInit);
		ShutdownPlayFabCore(bForceRunSync);
		bIsPlayFabCoreInit = false;
#else
		check(bIsPlayFabPartyInit || bIsPlayFabPartyXboxInit);
#endif

		Logout();

		if (bIsPlayFabPartyInit)
		{
			CleanupPlayFabPartySingleton();
			bIsPlayFabPartyInit = false;
		}

		if (bIsPlayFabPartyXboxInit)
		{
			CleanupPlayFabPartyXboxSingleton();
			bIsPlayFabPartyXboxInit = false;
		}
	}
	else
	{

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
		check(bIsPlayFabPartyInit || bIsPlayFabPartyXboxInit || bIsPlayFabCoreInit);
		ShutdownPlayFabCore(bForceRunSync);
		bIsPlayFabCoreInit = false;
#else
		check(bIsPlayFabPartyInit || bIsPlayFabPartyXboxInit);
#endif

		Logout();

		if (bIsPlayFabPartyInit)
		{
			HRESULT ShutdownPFPartySubmitTaskResult = XTaskQueueSubmitCallback(
				FGDKAsyncTaskQueue::GetGenericSerialQueue(),
				XTaskQueuePort::Work,
				nullptr,
				[](void* context, bool cancel)
				{
					CleanupPlayFabPartySingleton();
				});

			bIsPlayFabPartyInit = false; // We notify that PlayFabParty is not init even before the callback completed to prevent usage during shutdown, we pretend that is shutdown even if the internal task fails.

			if (!SUCCEEDED(ShutdownPFPartySubmitTaskResult))
			{
				UE_LOGF(LogPlayFabParty, Error, "PlayFabParty failed to upload Shutdown task to process queue. ErrorCode=[%0.8x] ErrorMessage=[%ls]", ShutdownPFPartySubmitTaskResult, *GetPlayFabPartyErrorMessage(ShutdownPFPartySubmitTaskResult));
			}
		}

		if (bIsPlayFabPartyXboxInit)
		{

			HRESULT ShutdownPFPartyXboxSubmitTaskResult = XTaskQueueSubmitCallback(
				FGDKAsyncTaskQueue::GetGenericSerialQueue(),
				XTaskQueuePort::Work,
				nullptr,
				[](void* context, bool cancel)
				{
					CleanupPlayFabPartyXboxSingleton();
				});

			bIsPlayFabPartyXboxInit = false; // We notify that PlayFabPartyXbox is not init even before the callback completed to prevent usage during shutdown, we pretend that is shutdown even if the internal task fails.

			if (!SUCCEEDED(ShutdownPFPartyXboxSubmitTaskResult))
			{
				UE_LOGF(LogPlayFabParty, Warning, "PlayFabPartyXbox failed to upload Shutdown task to process queue. ErrorCode=[%0.8x] ErrorMessage=[%ls]", ShutdownPFPartyXboxSubmitTaskResult, *GetPlayFabPartyErrorMessage(ShutdownPFPartyXboxSubmitTaskResult));
			}
		}
	}

	if (TickHandle.IsValid())
	{
		// Stop ticking if we're not initialized
		FTSTicker::GetCoreTicker().RemoveTicker(TickHandle);
		TickHandle.Reset();
	}

	QOSData.Reset();
	
}

void FPlayFabPartyManager::CheckLoginStatus()
{
	if (LocalUserLoginState == EPlayFabPartyLoginState::InProgress)
	{
		// A login is already in process
		return;
	}

	const double CurrentTimeSeconds = FPlatformTime::Seconds();

	if (LocalUserLoginState == EPlayFabPartyLoginState::LoggedIn)
	{

// Token refresh is handled automatically after 251000
#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE
		// Make sure we actually need to refresh our login
		if (CurrentTimeSeconds < NextLoginRefreshTimeSeconds)
		{
			// Need to wait longer to refresh our login
			return;
		}

		// Reset our delay
		NextLoginRefreshTimeSeconds = FPlatformTime::Seconds() + LoginRefreshDelaySeconds;

		// Refresh our login credentials
		RefreshLogin();
#endif
	}
	else
	{
		// Make sure we don't login too frequently
		if (CurrentTimeSeconds < NextLoginTimeSeconds)
		{
			// Need to wait longer
			return;
		}

		// Just get the first user available
		FPlatformUserId FirstUser = FPlatformMisc::GetPlatformUserForUserIndex(0);
		TOptional<uint64> LocalXboxUserId = IGDKRuntimeModule::Get().GetXUserIdByPlatformId(FirstUser);
		if (!LocalXboxUserId.IsSet())
		{
			// No local users
			return;
		}

		Login(LocalXboxUserId.GetValue());
	}
}

void FPlayFabPartyManager::Login(const uint64 XboxUserId)
{
	check(LocalUserLoginState == EPlayFabPartyLoginState::NeedLogin);

	UE_LOG(LogPlayFabParty, Log, TEXT("Logging into PlayFabParty for user %" UINT64_FMT), XboxUserId);

	const PartyError CreateXboxLocalChatUserError = Party::PartyXblManager::GetSingleton().CreateLocalChatUser(XboxUserId, nullptr, &PartyXboxLocalChatUser);
	if (PARTY_FAILED(CreateXboxLocalChatUserError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to create Xbox local chat user for PlayFabParty. ErrorCode=[%u] Error=[%ls]", CreateXboxLocalChatUserError, *GetPlayFabPartyErrorMessage(CreateXboxLocalChatUserError));

		Logout();
		return;
	}

	LocalUserXuid = XboxUserId;
	LocalUserLoginState = EPlayFabPartyLoginState::InProgress;
}

void FPlayFabPartyManager::RefreshLogin()
{
	check(LocalUserLoginState == EPlayFabPartyLoginState::LoggedIn);
	check(LocalUserXuid > 0);
	check(PartyXboxLocalChatUser);

	const PartyError LoginToPlayFabError = Party::PartyXblManager::GetSingleton().LoginToPlayFab(PartyXboxLocalChatUser, nullptr);
	if (PARTY_FAILED(LoginToPlayFabError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to login to PlayFabParty. ErrorCode=[%u] Error=[%ls]", LoginToPlayFabError, *GetPlayFabPartyErrorMessage(LoginToPlayFabError));

		Logout();
		return;
	}
}

void FPlayFabPartyManager::Logout()
{
	if (!bIsPlayFabPartyInit)
	{
		check(!PartyLocalUser);
		check(!PartyXboxLocalChatUser);
		check(LocalUserXuid == 0);
		check(LocalUserLoginState == EPlayFabPartyLoginState::NeedLogin);
		return;
	}

	if (PartyLocalUser)
	{
		const PartyError DestroyLocalUserError = Party::PartyManager::GetSingleton().DestroyLocalUser(PartyLocalUser, nullptr);
		if (PARTY_FAILED(DestroyLocalUserError))
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to destroy PlayFabParty local user. ErrorCode=[%u] Error=[%ls]", DestroyLocalUserError, *GetPlayFabPartyErrorMessage(DestroyLocalUserError));
		}
		PartyLocalUser = nullptr;
	}

	if (PartyXboxLocalChatUser)
	{
		const PartyError DestroyXboxLocalChatUserError = Party::PartyXblManager::GetSingleton().DestroyChatUser(PartyXboxLocalChatUser);
		if (PARTY_FAILED(DestroyXboxLocalChatUserError))
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to destroy PlayFabParty Xbox local chat user. ErrorCode=[%u] Error=[%ls]", DestroyXboxLocalChatUserError, *GetPlayFabPartyErrorMessage(DestroyXboxLocalChatUserError));
		}
		PartyXboxLocalChatUser = nullptr;
	}

	LocalUserXuid = 0;
	LocalUserLoginState = EPlayFabPartyLoginState::NeedLogin;
#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE // Token refresh is handled automatically after 251000
	NextLoginRefreshTimeSeconds = 0.0;
#endif
}

void FPlayFabPartyManager::HandleAppSuspended()
{
	// We are now suspending, so block initialization
	bIsAppSuspending = true;

	Logout();

	// Calling UpdatePlayFabPartyStatus now will cleanup playfab
	UpdatePlayFabPartyStatus();
}

void FPlayFabPartyManager::HandleAppResume()
{
	// We are no longer suspending
	bIsAppSuspending = false;

	// Attempt to reinit playfab now (may not happen now if the network isn't ready yet)
	UpdatePlayFabPartyStatus();
}

void FPlayFabPartyManager::HandleNetworkInitializationChanged(bool bNetworkInitializationStatus)
{
	// Update our current network state
	bIsNetworkReady = bNetworkInitializationStatus;

	// Potentially Init or Shutdown playfab now that our network changed
	UpdatePlayFabPartyStatus();
}

void FPlayFabPartyManager::HandleQOSDataRequested(FName SessionName)
{
	IOnlineSubsystem* OnlineSubsystemGDK = IOnlineSubsystem::Get(GDK_SUBSYSTEM);
	if (!OnlineSubsystemGDK)
	{
		UE_LOGF(LogPlayFabParty, Error, "Failed to write QOS data due to null OSS. SessionName=[%ls]", *SessionName.ToString());
		return;
	}

	IOnlineSessionPtr OnlineSessionInterface = OnlineSubsystemGDK->GetSessionInterface();
	if (!OnlineSessionInterface.IsValid())
	{
		UE_LOGF(LogPlayFabParty, Error, "Failed to write QOS data due to null OSS session interface. SessionName=[%ls]", *SessionName.ToString());
		return;
	}

	FNamedOnlineSession* Session = OnlineSessionInterface->GetNamedSession(SessionName);
	if (!Session)
	{
		UE_LOGF(LogPlayFabParty, Error, "Failed to write QOS data due to missing named session. SessionName=[%ls]", *SessionName.ToString());
		return;
	}

	// Write QOS string into session settings
	Session->SessionSettings.Set(SETTING_QOS, QOSData.Get(FString()), EOnlineDataAdvertisementType::DontAdvertise);

	if (QOSData.IsSet())
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Wrote QOS data to session settings successfully. SessionName=[%ls]", *SessionName.ToString());
	}
	else
	{
		UE_LOGF(LogPlayFabParty, Warning, "Wrote Empty QOS data to session settings due to empty cached value. SessionName=[%ls]", *SessionName.ToString());
	}
}

bool FPlayFabPartyManager::Tick(float DeltaSeconds)
{
	if (bIsPlayFabPartyInit && bIsPlayFabPartyXboxInit)
	{
		ProcessPlayFabPartyEvents();
		ProcessPlayFabPartyXboxEvents();

		CheckLoginStatus();
	}

	return true;
}

void FPlayFabPartyManager::ProcessPlayFabPartyEvents()
{
	uint32 StateChangeCount = 0u;
	Party::PartyStateChangeArray StateChangesArray = nullptr;

	Party::PartyManager& PlayFabPartyManager = Party::PartyManager::GetSingleton();

	const PartyError StartProcessingStateChangesResult = PlayFabPartyManager.StartProcessingStateChanges(&StateChangeCount, &StateChangesArray);
	if (PARTY_FAILED(StartProcessingStateChangesResult))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start processing PlayFabParty state changes. ErrorCode=[%u] Error=[%ls]", StartProcessingStateChangesResult, *GetPlayFabPartyErrorMessage(StartProcessingStateChangesResult));
		return;
	}

	for (uint32 StateIndex = 0; StateIndex < StateChangeCount; ++StateIndex)
	{
		const Party::PartyStateChange* StateChange = StateChangesArray[StateIndex];
		if (!ensure(StateChange))
		{
			continue;
		}

		switch (StateChange->stateChangeType)
		{
			case Party::PartyStateChangeType::RegionsChanged:
				// We have received region information from Azure
				HandleRegionsChanged(static_cast<const Party::PartyRegionsChangedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::ConnectToNetworkCompleted:
				// We have finished connecting to a network
				HandleConnectToNetworkCompleted(static_cast<const Party::PartyConnectToNetworkCompletedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::AuthenticateLocalUserCompleted:
				// We have finished authenticating with a network
				HandleAuthenticateLocalUserCompleted(static_cast<const Party::PartyAuthenticateLocalUserCompletedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::CreateEndpointCompleted:
				// We have finished creating an endpoint
				HandleCreateEndpointCompleted(static_cast<const Party::PartyCreateEndpointCompletedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::EndpointDestroyed:
				// A client left the network
				HandleEndpointDestroyed(static_cast<const Party::PartyEndpointDestroyedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::NetworkDestroyed:
				// Our network has been destroyed.
				HandleNetworkDestroyed(static_cast<const Party::PartyNetworkDestroyedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::EndpointMessageReceived:
				// We received a data message
				HandleEndpointMessageReceived(static_cast<const Party::PartyEndpointMessageReceivedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::CreateInvitationCompleted:
				// An invitation has finished
				HandleCreateInvitationCompleted(static_cast<const Party::PartyCreateInvitationCompletedStateChange*>(StateChange));
				break;
			case Party::PartyStateChangeType::LeaveNetworkCompleted:
				// Finished leaving party network
				HandleLeaveNetworkCompleted(static_cast<const Party::PartyLeaveNetworkCompletedStateChange*>(StateChange));
				break;

			// Just Log these events, we don't need to do anything for them
			case Party::PartyStateChangeType::DestroyLocalUserCompleted:
			case Party::PartyStateChangeType::CreateNewNetworkCompleted:
			case Party::PartyStateChangeType::NetworkConfigurationMadeAvailable:
			case Party::PartyStateChangeType::NetworkDescriptorChanged:
			case Party::PartyStateChangeType::LocalUserRemoved:
			case Party::PartyStateChangeType::RemoveLocalUserCompleted:
			case Party::PartyStateChangeType::LocalUserKicked:
			case Party::PartyStateChangeType::DestroyEndpointCompleted:
			case Party::PartyStateChangeType::EndpointCreated:
			case Party::PartyStateChangeType::RemoteDeviceCreated:
			case Party::PartyStateChangeType::RemoteDeviceDestroyed:
			case Party::PartyStateChangeType::RemoteDeviceJoinedNetwork:
			case Party::PartyStateChangeType::RemoteDeviceLeftNetwork:
			case Party::PartyStateChangeType::DevicePropertiesChanged:
			case Party::PartyStateChangeType::DataBuffersReturned:
			case Party::PartyStateChangeType::EndpointPropertiesChanged:
			case Party::PartyStateChangeType::SynchronizeMessagesBetweenEndpointsCompleted:
			case Party::PartyStateChangeType::RevokeInvitationCompleted:
			case Party::PartyStateChangeType::InvitationCreated:
			case Party::PartyStateChangeType::InvitationDestroyed:
			case Party::PartyStateChangeType::NetworkPropertiesChanged:
			case Party::PartyStateChangeType::KickDeviceCompleted:
			case Party::PartyStateChangeType::KickUserCompleted:
			case Party::PartyStateChangeType::CreateChatControlCompleted:
			case Party::PartyStateChangeType::DestroyChatControlCompleted:
			case Party::PartyStateChangeType::ChatControlCreated:
			case Party::PartyStateChangeType::ChatControlDestroyed:
			case Party::PartyStateChangeType::SetChatAudioEncoderBitrateCompleted:
			case Party::PartyStateChangeType::ChatTextReceived:
			case Party::PartyStateChangeType::VoiceChatTranscriptionReceived:
			case Party::PartyStateChangeType::SetChatAudioInputCompleted:
			case Party::PartyStateChangeType::SetChatAudioOutputCompleted:
			case Party::PartyStateChangeType::LocalChatAudioInputChanged:
			case Party::PartyStateChangeType::LocalChatAudioOutputChanged:
			case Party::PartyStateChangeType::SetTextToSpeechProfileCompleted:
			case Party::PartyStateChangeType::SynthesizeTextToSpeechCompleted:
			case Party::PartyStateChangeType::SetLanguageCompleted:
			case Party::PartyStateChangeType::SetTranscriptionOptionsCompleted:
			case Party::PartyStateChangeType::SetTextChatOptionsCompleted:
			case Party::PartyStateChangeType::ChatControlPropertiesChanged:
			case Party::PartyStateChangeType::ChatControlJoinedNetwork:
			case Party::PartyStateChangeType::ChatControlLeftNetwork:
			case Party::PartyStateChangeType::ConnectChatControlCompleted:
			case Party::PartyStateChangeType::DisconnectChatControlCompleted:
			case Party::PartyStateChangeType::PopulateAvailableTextToSpeechProfilesCompleted:
			case Party::PartyStateChangeType::ConfigureAudioManipulationVoiceStreamCompleted:
			case Party::PartyStateChangeType::ConfigureAudioManipulationCaptureStreamCompleted:
			case Party::PartyStateChangeType::ConfigureAudioManipulationRenderStreamCompleted:
				UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab Unhandled(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));
				break;
		}
	}

	const PartyError StopProcessingStateChangesResult = PlayFabPartyManager.FinishProcessingStateChanges(StateChangeCount, StateChangesArray);
	if (PARTY_FAILED(StopProcessingStateChangesResult))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to finish processing PlayFabParty state changes. ErrorCode=[%u] Error=[%ls]", StopProcessingStateChangesResult, *GetPlayFabPartyErrorMessage(StopProcessingStateChangesResult));
	}
}

void FPlayFabPartyManager::HandleRegionsChanged(const Party::PartyRegionsChangedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab RegionsChanged(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	if (StateChange->result != Party::PartyStateChangeResult::Succeeded)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to query region information from Azure. ErrorCode=[%u] Error=[%ls]", StateChange->errorDetail, *GetPlayFabPartyErrorMessage(StateChange->errorDetail));
		return;
	}

	uint32 RegionCount = 0;
	const Party::PartyRegion* RegionList = nullptr;
	const PartyError GetRegionsError = Party::PartyManager::GetSingleton().GetRegions(&RegionCount, &RegionList);
	if (PARTY_FAILED(GetRegionsError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to get region information from PlayFab. ErrorCode=[%u] Error=[%ls]", GetRegionsError, *GetPlayFabPartyErrorMessage(GetRegionsError));
		return;
	}

	// Build QOS string
	FString QOSString;
	FJsonWriter JsonWriter = FJsonWriterFactory::Create(&QOSString, 0);
	JsonWriter->WriteObjectStart();
	for (uint32 Index = 0; Index < RegionCount; ++Index)
	{
		const Party::PartyRegion* Region = &RegionList[Index];

		JsonWriter->WriteObjectStart(FString(ANSI_TO_TCHAR(Region->regionName)));
		{
			JsonWriter->WriteValue(TEXT("latency"), static_cast<int64>(Region->roundTripLatencyInMilliseconds));
		}
		JsonWriter->WriteObjectEnd();
	}
	JsonWriter->WriteObjectEnd();
	JsonWriter->Close();

	// Save QOS string for sessions to use later
	QOSData.Emplace(MoveTemp(QOSString));
}

void FPlayFabPartyManager::HandleConnectToNetworkCompleted(const Party::PartyConnectToNetworkCompletedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab ConnectToNetworkCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	if (!StateChange->asyncIdentifier)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Received network connected event with invalid socket context");
		return;
	}

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(reinterpret_cast<uint64>(StateChange->asyncIdentifier));
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received network connected event after socket went away");
		return;
	}

	if (!Socket->IsServer())
	{
		bIsNetworkConnected = true;
	}

	if (StateChange->result != Party::PartyStateChangeResult::Succeeded)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to connect to PlayFabParty network. Result=[%u] ErrorCode=[%u] Error=[%ls]", static_cast<uint32>(StateChange->result), StateChange->errorDetail, *GetPlayFabPartyErrorMessage(StateChange->errorDetail));
		Socket->Close();
		return;
	}

	// Store the mapping of this network to the socket context it belongs to
	PartyToSocketContextMap.Add(StateChange->network, Socket->GetContext());

	// Tell our socket we've connected successfully
	Socket->HandleConnectToNetworkComplete();
}

void FPlayFabPartyManager::HandleAuthenticateLocalUserCompleted(const Party::PartyAuthenticateLocalUserCompletedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab AuthenticateLocalUserCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	if (!StateChange->asyncIdentifier)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Received local user auth event with invalid socket context");
		return;
	}

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(reinterpret_cast<uint64>(StateChange->asyncIdentifier));
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received local user auth event after socket went away");
		return;
	}

	if (StateChange->result != Party::PartyStateChangeResult::Succeeded)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to authenticate with PlayFabParty network. Result=[%u] ErrorCode=[%u] Error=[%ls]", static_cast<uint32>(StateChange->result), StateChange->errorDetail, *GetPlayFabPartyErrorMessage(StateChange->errorDetail));
		if (StateChange->result == Party::PartyStateChangeResult::UserNotAuthorized)
		{
			Socket->HandleAuthorizationFailure();
			return;
		}
		Socket->Close();
		return;
	}
	Socket->HandleLocalUserAuthenticationComplete();
}

void FPlayFabPartyManager::HandleCreateEndpointCompleted(const Party::PartyCreateEndpointCompletedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab CreateEndpointCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	if (!StateChange->asyncIdentifier)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Received create endpoint completed event with invalid socket context");
		return;
	}

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(reinterpret_cast<uint64>(StateChange->asyncIdentifier));
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received create endpoint completed event after socket went away");
		return;
	}

	if (StateChange->result != Party::PartyStateChangeResult::Succeeded)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to create local endpoint with PlayFabParty network. Result=[%u] ErrorCode=[%u] Error=[%ls]", static_cast<uint32>(StateChange->result), StateChange->errorDetail, *GetPlayFabPartyErrorMessage(StateChange->errorDetail));
		Socket->Close();
		return;
	}

	Socket->HandleCreateLocalEndpointComplete();
}

void FPlayFabPartyManager::HandleEndpointDestroyed(const Party::PartyEndpointDestroyedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab EndpointDestroyed(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	uint64* SocketContextPtr = PartyToSocketContextMap.Find(StateChange->network);
	if (!SocketContextPtr)
	{
		// We've already forgotten about this network (may have just left)
		UE_LOGF(LogPlayFabParty, Verbose, "Received endpoint destroyed from unknown network. Address=[%ls]", *FPlayFabPartyInternetAddr(*StateChange->endpoint).ToString(true));
		return;
	}

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(*SocketContextPtr);
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received endpoint destroyed event after socket went away");
		return;
	}


	FPlayFabPartyInternetAddr SocketAddress;
	Socket->GetAddress(SocketAddress);

	const FPlayFabPartyInternetAddr DestroyedAddress(*StateChange->endpoint);

	if (SocketAddress == DestroyedAddress)
	{
		// TODO: bubble up host disconnect?
		Socket->Close();
	}
}

void FPlayFabPartyManager::HandleNetworkDestroyed(const Party::PartyNetworkDestroyedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab NetworkDestroyed(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	uint64 SocketContext = 0;
	if (!PartyToSocketContextMap.RemoveAndCopyValue(StateChange->network, SocketContext))
	{
		// We've already forgotten about this network
		return;
	}

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(SocketContext);
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received network destroyed event after socket went away");
		return;
	}

	// TODO: bubble up DestroyedReason

	Socket->Close();
}

void FPlayFabPartyManager::HandleEndpointMessageReceived(const Party::PartyEndpointMessageReceivedStateChange* const StateChange)
{
	uint64* SocketContextPtr = PartyToSocketContextMap.Find(StateChange->network);
	if (!SocketContextPtr)
	{
		check(StateChange->senderEndpoint != nullptr);

		// We've already forgotten about this network (may have just left)
		UE_LOGF(LogPlayFabParty, Verbose, "Received packet from unknown network endpoint. Address=[%ls]", *FPlayFabPartyInternetAddr(*StateChange->senderEndpoint).ToString(true));
		return;
	}

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(*SocketContextPtr);
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received endpoint message event after socket went away");
		return;
	}

	FPlayFabPartyPendingPacket Packet;
	Packet.Data.Append(static_cast<const uint8*>(StateChange->messageBuffer), StateChange->messageSize);
	if (ensure(StateChange->senderEndpoint))
	{
		Packet.Sender = FPlayFabPartyInternetAddr(*StateChange->senderEndpoint);
	}

	Socket->HandleEndpointMessageReceived(MoveTemp(Packet));
}

void FPlayFabPartyManager::HandleCreateInvitationCompleted(const Party::PartyCreateInvitationCompletedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab CreateInvitationCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(reinterpret_cast<uint64>(StateChange->asyncIdentifier));
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received CreateInvitationCompleted event after socket went away");
		return;
	}

	Socket->HandleCreateInvitationCompleted(StateChange->invitation, StateChange->result == Party::PartyStateChangeResult::Succeeded);
}

void FPlayFabPartyManager::HandleLeaveNetworkCompleted(const Party::PartyLeaveNetworkCompletedStateChange* StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFab PartyLeaveNetworkCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	bIsNetworkConnected = false;
}

void FPlayFabPartyManager::ProcessPlayFabPartyXboxEvents()
{
	GDK_SCOPE_NOT_TIME_SENSITIVE(); //PartyXmlManager::StartProcessingStateChanges ultimately calls XTaskQueueCreate, which should not be called on time-sensitive threads

	uint32 StateChangeCount = 0u;
	Party::PartyXblStateChangeArray StateChangesArray = nullptr;

	Party::PartyXblManager& PlayFabPartyXblManager = Party::PartyXblManager::GetSingleton();

	// Get PlayFabParty Xbox Events
	const PartyError StartProcessingStateChangesResult = PlayFabPartyXblManager.StartProcessingStateChanges(&StateChangeCount, &StateChangesArray);
	if (PARTY_FAILED(StartProcessingStateChangesResult))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start processing PlayFabParty Xbox state changes. ErrorCode=[%u] Error=[%ls]", StartProcessingStateChangesResult, *GetPlayFabPartyErrorMessage(StartProcessingStateChangesResult));
		return;
	}

	// Process PlayFabParty Xbox Events
	for (uint32 StateIndex = 0; StateIndex < StateChangeCount; ++StateIndex)
	{
		const Party::PartyXblStateChange* StateChange = StateChangesArray[StateIndex];
		if (!ensure(StateChange))
		{
			continue;
		}

		switch (StateChange->stateChangeType)
		{
			case Party::PartyXblStateChangeType::CreateLocalChatUserCompleted:
				// We have successfully created our local chat user
				HandleXboxCreateLocalChatUserCompleted(static_cast<const Party::PartyXblCreateLocalChatUserCompletedStateChange*>(StateChange));
				break;
			case Party::PartyXblStateChangeType::LoginToPlayFabCompleted:
				// Our login call to PlayFab has completed
				HandleLoginToPlayFabCompleted(static_cast<const Party::PartyXblLoginToPlayFabCompletedStateChange*>(StateChange));
				break;
			case Party::PartyXblStateChangeType::GetEntityIdsFromXboxLiveUserIdsCompleted:
				// Our query for Xbox Live User IDs has completed
				HandleGetEntityIdsFromXboxLiveUserIdsCompleted(static_cast<const Party::PartyXblGetEntityIdsFromXboxLiveUserIdsCompletedStateChange*>(StateChange));
				break;

			// Just Log these events
			case Party::PartyXblStateChangeType::LocalChatUserDestroyed:
			case Party::PartyXblStateChangeType::RequiredChatPermissionInfoChanged:
			case Party::PartyXblStateChangeType::TokenAndSignatureRequested:
				UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFabParty Xbox Unhandled(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));
				break;
		}
	}

	// Cleanup PlayFabParty Xbox Events
	const PartyError StopProcessingStateChangesResult = PlayFabPartyXblManager.FinishProcessingStateChanges(StateChangeCount, StateChangesArray);
	if (PARTY_FAILED(StopProcessingStateChangesResult))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to finish processing PlayFabParty Xbox state changes. ErrorCode=[%u] Error=[%ls]", StopProcessingStateChangesResult, *GetPlayFabPartyErrorMessage(StopProcessingStateChangesResult));
	}
}

void FPlayFabPartyManager::HandleXboxCreateLocalChatUserCompleted(const Party::PartyXblCreateLocalChatUserCompletedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFabParty Xbox XboxCreateLocalChatUserCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	if (StateChange->result != Party::PartyXblStateChangeResult::Succeeded)
	{
		UE_LOGF(LogPlayFabParty, Warning, "PlayFabParty create local chat user failed. Result=[%u] ErrorCode=[%u] Error=[%ls]", EnumToUnderlyingType(StateChange->result), StateChange->errorDetail, *GetPlayFabPartyErrorMessage(StateChange->errorDetail));

		Logout();
		return;
	}

	UE_LOGF(LogPlayFabParty, Log, "Successfully created PlayFabXbox Local Chat User");

	const PartyError LoginToPlayFabError = Party::PartyXblManager::GetSingleton().LoginToPlayFab(PartyXboxLocalChatUser, nullptr);
	if (PARTY_FAILED(LoginToPlayFabError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to login to PlayFabParty. ErrorCode=[%u] Error=[%ls]", LoginToPlayFabError, *GetPlayFabPartyErrorMessage(LoginToPlayFabError));

		Logout();
		return;
	}
}

#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE
void FPlayFabPartyManager::HandleLocalPlayerCreationWithEntityToken(const Party::PartyXblLoginToPlayFabCompletedStateChange* const StateChange)
{
	if (LocalUserLoginState == EPlayFabPartyLoginState::LoggedIn && PartyLocalUser)
	{
		// Token refresh
		const PartyError UpdateTokenError = PartyLocalUser->UpdateEntityToken(StateChange->titlePlayerEntityToken);
		if (PARTY_FAILED(UpdateTokenError))
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to update our PlayFabParty local user token. ErrorCode=[%u] Error=[%ls]", UpdateTokenError, *GetPlayFabPartyErrorMessage(UpdateTokenError));
		}
		else
		{
			UE_LOG(LogPlayFabParty, Log, TEXT("Successfully refreshed login token for %") UINT64_FMT, LocalUserXuid);
		}
	}
	else if (LocalUserLoginState == EPlayFabPartyLoginState::InProgress)
	{
		// New Login
		UE_LOG(LogPlayFabParty, Log, TEXT("Successfully logged into PlayFabParty for %") UINT64_FMT, LocalUserXuid);

		const PartyError CreateLocalUserError = Party::PartyManager::GetSingleton().CreateLocalUser(StateChange->entityId, StateChange->titlePlayerEntityToken, &PartyLocalUser);
		if (PARTY_FAILED(CreateLocalUserError))
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to create our PlayFabParty local user. ErrorCode=[%u] Error=[%ls]", CreateLocalUserError, *GetPlayFabPartyErrorMessage(CreateLocalUserError));

			NextLoginTimeSeconds = FPlatformTime::Seconds() + LoginFailureDelaySeconds;
			Logout();
			return;
		}

		NextLoginRefreshTimeSeconds = FPlatformTime::Seconds() + LoginRefreshDelaySeconds;
		LocalUserLoginState = EPlayFabPartyLoginState::LoggedIn;
	}
}
#endif

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
void FPlayFabPartyManager::HandleLocalPlayerCreationWithEntityHandle()
{
	if (LocalUserLoginState == EPlayFabPartyLoginState::InProgress)
	{
		// New Login
		UE_LOG(LogPlayFabParty, Log, TEXT("Successfully logged into PlayFabParty for %") UINT64_FMT, LocalUserXuid);

		HRESULT CreateLocalUserResult = AsyncGDKTask(
			[weakThis = this->AsWeak()](XAsyncBlock* PFCoreLoginAsyncBlock) -> HRESULT
			{
				if (TSharedPtr<FPlayFabPartyManager> PFPartyManagerPtr = weakThis.Pin())
				{
					FGDKUserHandle UserHandle = IGDKRuntimeModule::Get().GetUserHandleByXUserId(PFPartyManagerPtr->LocalUserXuid);
					PFAuthenticationLoginWithXUserRequest LoginRequest{};
					LoginRequest.createAccount = true;
					LoginRequest.user = UserHandle;
					HRESULT Result = PFAuthenticationLoginWithXUserAsync(PFPartyManagerPtr->PFCoreServiceHandle, &LoginRequest, PFCoreLoginAsyncBlock);

					return Result;
				}

				UE_LOGF(LogPlayFabParty, Warning, "[PlayFabCoreCreateLocalUser StartGDKTask] PlayFabManager was destroyed before the async block was executed");
				return E_ABORT;
			},
			[weakThis = this->AsWeak()](XAsyncBlock* PFCoreLoginAsyncBlock)
			{
				TSharedPtr<FPlayFabPartyManager> PFPartyManagerPtr = weakThis.Pin();
				if (!PFPartyManagerPtr)
				{
					UE_LOGF(LogPlayFabParty, Warning, "[PlayFabCoreCreateLocalUser GDKTaskCallback] PlayFabManager was destroyed before the async block was executed");
					return;
				}

				if (PFPartyManagerPtr->LocalUserLoginState !=  EPlayFabPartyLoginState::InProgress)
				{
					UE_LOGF(LogPlayFabParty, Log, "[PlayFabCoreCreateLocalUser GDKTaskCallback] Current user was logged out before login operation completed");
					return;
				}

				std::vector<char> LoginResultBuffer;
				PFAuthenticationLoginResult const* LoginResult;
				size_t BufferSize;
				HRESULT PFCoreLoginResult = PFAuthenticationLoginWithXUserGetResultSize(PFCoreLoginAsyncBlock, &BufferSize);
				if (FAILED(PFCoreLoginResult))
				{
					UE_LOGF(LogPlayFabParty, Warning, "PlayFabCore failed to get login result size. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreLoginResult, *GetPlayFabPartyErrorMessage(PFCoreLoginResult));
					PFPartyManagerPtr->NextLoginTimeSeconds = FPlatformTime::Seconds() + PFPartyManagerPtr->LoginFailureDelaySeconds;
					PFPartyManagerPtr->Logout();
					return;
				}

				LoginResultBuffer.resize(BufferSize);
				PFEntityHandle EntityHandle{ nullptr };
				PFCoreLoginResult = PFAuthenticationLoginWithXUserGetResult(PFCoreLoginAsyncBlock, &EntityHandle, LoginResultBuffer.size(), LoginResultBuffer.data(), &LoginResult, nullptr);
				if (FAILED(PFCoreLoginResult))
				{
					UE_LOGF(LogPlayFabParty, Warning, "PlayFabCore failed to get login result data. ErrorCode=[%0.8x] ErrorMessage=[%ls]", PFCoreLoginResult, *GetPlayFabPartyErrorMessage(PFCoreLoginResult));
					PFPartyManagerPtr->NextLoginTimeSeconds = FPlatformTime::Seconds() + PFPartyManagerPtr->LoginFailureDelaySeconds;
					PFPartyManagerPtr->Logout();
					return;
				}

				const PartyError CreateLocalUserError = Party::PartyManager::GetSingleton().CreateLocalUser(EntityHandle, &PFPartyManagerPtr->PartyLocalUser);
				if (PARTY_FAILED(CreateLocalUserError))
				{
					UE_LOGF(LogPlayFabParty, Warning, "PlayFabCore failed to create local user. ErrorCode=[%0.8x] ErrorMessage=[%ls]", CreateLocalUserError, *GetPlayFabPartyErrorMessage(CreateLocalUserError));
					PFPartyManagerPtr->NextLoginTimeSeconds = FPlatformTime::Seconds() + PFPartyManagerPtr->LoginFailureDelaySeconds;
					PFPartyManagerPtr->Logout();
					return;
				}

				UE_LOGF(LogPlayFabParty, Log, "PlayFabCore login operation SUCCEEDED for XBOX user. PlayerID -> %ls", *FString(LoginResult->playFabId));

				PFPartyManagerPtr->XboxUserIdToPlayFabEntityHandleMap.Add(PFPartyManagerPtr->LocalUserXuid, EntityHandle);
				PFPartyManagerPtr->LocalUserLoginState = EPlayFabPartyLoginState::LoggedIn;
			}
		, FGDKAsyncTaskQueue::GetGenericSerialQueue());

		if (FAILED(CreateLocalUserResult))
		{
			UE_LOGF(LogPlayFabParty, Warning, "PlayFabCore failed to start login process for user. ErrorCode=[%0.8x] ErrorMessage=[%ls]", CreateLocalUserResult, *GetPlayFabPartyErrorMessage(CreateLocalUserResult));
			NextLoginTimeSeconds = FPlatformTime::Seconds() + LoginFailureDelaySeconds;
			Logout();
		}
	}
}
#endif

void FPlayFabPartyManager::HandleLoginToPlayFabCompleted(const Party::PartyXblLoginToPlayFabCompletedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFabParty Xbox LoginToPlayFabCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	if (StateChange->result != Party::PartyXblStateChangeResult::Succeeded)
	{
		UE_LOGF(LogPlayFabParty, Warning, "PlayFabParty login failed. Result=[%u] ErrorCode=[%u] Error=[%ls]", EnumToUnderlyingType(StateChange->result), StateChange->errorDetail, *GetPlayFabPartyErrorMessage(StateChange->errorDetail));

		NextLoginTimeSeconds = FPlatformTime::Seconds() + LoginFailureDelaySeconds;
		Logout();
		return;
	}

	// Cache our local entity id
	if (LocalUserXuid > 0 && XboxUserIdToPlayFabEntityIdMap.Find(LocalUserXuid) == nullptr)
	{
		XboxUserIdToPlayFabEntityIdMap.Emplace(LocalUserXuid, StateChange->entityId);
	}

	// In GDK >= 251000 this EntityToken is the same one as the one retrieved using EntityHandle after creating a local user. We are good storing this one even before getting an EntityHandle.
	XboxUserIdToPlayFabEntityTokenMap.Emplace(LocalUserXuid, StateChange->titlePlayerEntityToken); 

#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE
	HandleLocalPlayerCreationWithEntityToken(StateChange);
#else
	HandleLocalPlayerCreationWithEntityHandle();
#endif
}

void FPlayFabPartyManager::HandleGetEntityIdsFromXboxLiveUserIdsCompleted(const Party::PartyXblGetEntityIdsFromXboxLiveUserIdsCompletedStateChange* const StateChange)
{
	UE_LOGF(LogPlayFabParty, Verbose, "Received PlayFabParty Xbox GetEntityIdsFromXboxLiveUserIdsCompleted(%u) event", EnumToUnderlyingType(StateChange->stateChangeType));

	const bool bWasSuccessful = StateChange->result == Party::PartyXblStateChangeResult::Succeeded;

	// Cache the values if we were successful
	if (bWasSuccessful)
	{
		for (uint32 Index = 0; Index < StateChange->entityIdMappingCount; ++Index)
		{
			const Party::PartyXblXboxUserIdToPlayFabEntityIdMapping& Mapping = StateChange->entityIdMappings[Index];
			if (!XboxUserIdToPlayFabEntityIdMap.Find(Mapping.xboxLiveUserId))
			{
				XboxUserIdToPlayFabEntityIdMap.Emplace(Mapping.xboxLiveUserId, Mapping.playfabEntityId);
			}
		}
	}
	else
	{
		UE_LOGF(LogPlayFabParty, Warning, "GetEntityIdsFromXboxLiveUserIds query failed. ErrorCode=[%u] Error=[%ls]", StateChange->errorDetail, *GetPlayFabPartyErrorMessage(StateChange->errorDetail));
	}

	FPlayFabPartySocket* Socket = SocketSubsystem.GetSocketFromContext(reinterpret_cast<uint64>(StateChange->asyncIdentifier));
	if (!Socket)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received GetEntityIdsFromXboxLiveUserIdsCompleted event after socket went away");
		return;
	}

	Socket->HandleGetEntityIdsFromXboxLiveUserIdsCompleted(bWasSuccessful);
}
#endif // WITH_PLAYFAB_PARTY
