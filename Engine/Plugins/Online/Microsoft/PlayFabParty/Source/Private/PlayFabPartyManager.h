// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_PLAYFAB_PARTY
#include "CoreMinimal.h"
#include "PlayFabParty.h"
#include "Containers/Ticker.h"

THIRD_PARTY_INCLUDES_START
#include "Microsoft/AllowMicrosoftPlatformTypes.h"
#include <grdk.h>
#include <XTaskQueue.h>
#include "Microsoft/HideMicrosoftPlatformTypes.h"
THIRD_PARTY_INCLUDES_END


namespace Party
{
	// PlayFab Event Types
	struct PartyRegionsChangedStateChange;
	struct PartyConnectToNetworkCompletedStateChange;
	struct PartyAuthenticateLocalUserCompletedStateChange;
	struct PartyCreateEndpointCompletedStateChange;
	struct PartyEndpointCreatedStateChange;
	struct PartyEndpointDestroyedStateChange;
	struct PartyNetworkDestroyedStateChange;
	struct PartyEndpointMessageReceivedStateChange;
	struct PartyCreateInvitationCompletedStateChange;

	// PlayFab Xbox Event Types
	struct PartyXblCreateLocalChatUserCompletedStateChange;
	struct PartyXblLoginToPlayFabCompletedStateChange;
	struct PartyXblTokenAndSignatureRequestedStateChange;
	struct PartyXblGetEntityIdsFromXboxLiveUserIdsCompletedStateChange;

	// PlayFab Network types
	class PartyNetwork;

	// PlayFab User types
	class PartyXblLocalChatUser;
	class PartyLocalUser;
}
class FPlayFabPartySocketSubsystem;
class FSocket;

using HANDLE = void*;

/** States of our login state machine */
enum class EPlayFabPartyLoginState : uint8
{
	/** We need to login still */
	NeedLogin = 0,
	/** We are currently logging in */
	InProgress = 1,
	/** We are successfully logged in */
	LoggedIn = 2,
};

/** Storage class for a PlayFab Entity ID */
struct FPlayFabPartyEntityId
{
public:
	explicit FPlayFabPartyEntityId(PartyString EntityId);

	PartyString GetEntityId() const;

private:
	TArray<char, TFixedAllocator<Party::c_maxEntityIdStringLength + 1>> Data;
};

/** Storage class for a PlayFab Entity ID */
struct FPlayFabPartyEntityToken
{
public:
	explicit FPlayFabPartyEntityToken(PartyString EntityToken);

	PartyString GetEntityToken() const;

private:
	TArray<char, TFixedAllocator<1024>> Data;

};


/** Class to manage the status of PlayFabParty and handle its events */
class FPlayFabPartyManager
	: public TSharedFromThis<FPlayFabPartyManager, ESPMode::ThreadSafe>
{
public:
	virtual ~FPlayFabPartyManager();

	/** Create an instance of our PlayFabParty manager */
	static TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> CreateManager(FPlayFabPartySocketSubsystem& OwningSocketSubsystem, const FString& AppId, FString& OutError);
	/** Get the PlayFabParty AppId specified in the configuration files */
	static TOptional<FString> GetAppId();

	/** Bind our system delegates*/
	bool BindSystemDelegates();
	/** Unbind our system delegates */
	void UnbindSystemDelegates();

	/** Check if PlayFabParty is Initialized */
	bool IsInitialized() const;
	/** We are ready if PlayFabParty is Initialized and we have a logged in user */
	bool IsReady() const;

	/** Get our Party Local User instance */
	Party::PartyLocalUser* GetPartyLocalUser();
	/** Get our Party Local User instance */
	const Party::PartyLocalUser* GetPartyLocalUser() const;

	/** Get our Party Xbox Local Chat User instance */
	Party::PartyXblLocalChatUser* GetPartyXboxLocalChatUser();
	/** Get our Party Xbox Local Chat User instance */
	const Party::PartyXblLocalChatUser* GetPartyXboxLocalChatUser() const;

	/** Get our QOS json string to be sent to Xbox Live */
	const TOptional<FString>& GetQOSString() const;

	/** Create a new network descriptor with the provided configuration */
	TUniquePtr<Party::PartyNetworkDescriptor> CreateNetwork(const FString& InitialInvitationString, const uint32 MaxPlayers, void* CallbackContext) const;

	/** Do we have a cached Entity ID for the provided Xuid? */
	bool HaveEntityIdForXuid(const uint64 Xuid);
	/** Get the cached Entity ID for the provided Xuid. Asserts if we do not have a value cached! */
	PartyString GetEntityIdForXuid(const uint64 Xuid);

	/** Get the cached Entity ID for the provided Xuid if any! */
	PartyString GetEntityTokenForXuid(const uint64 Xuid);

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
	/** Get the cached Entity Handle for the provided Xuid if any! */
	PFEntityHandle GetEntityHandleForXuid(const uint64 Xuid);
#endif

	/** Get PlayFab socket of listen server */
	TSharedPtr<FSocket> GetPlayFabSocketListenServer() const { return PlayFabSocketListenServer; }
	/** Save PlayFab socket of listen server so to prevent recreating */
	void SetPlayFabSocketListenServer(const TSharedPtr<FSocket>& Socket) { PlayFabSocketListenServer = Socket; }

	/** Check if PlayFab network is connected */
	bool IsNetworkConnected() const { return bIsNetworkConnected; }

protected:
	FPlayFabPartyManager(FPlayFabPartySocketSubsystem& SocketSubsystem, const FString& AppId);

	/** Reprocess our PlayFabParty state after a state change happens and either init, shutdown, or do nothing */
	void UpdatePlayFabPartyStatus();

	/** Attempt to initialize PlayFabParty */
	void InitPlayFabParty();

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
	/** Shutdown PlayFabCore */
	void ShutdownPlayFabCore(bool bForceRunSync = false);
#endif
	/** Shutdown PlayFabParty manager singleton */
	static void CleanupPlayFabPartySingleton();
	/** Shutdown PlayFabParty Xbox manager singleton */
	static void CleanupPlayFabPartyXboxSingleton();
	/** Shutdown PlayFabParty */
	void ShutdownPlayFabParty(bool bForceRunSync = false);

	/** Check if we should login or refresh our auth right now */
	void CheckLoginStatus();

	/** Login to PlayFabParty as the specified XUID */
	void Login(const uint64 XboxUserId);
	/** Refresh our current login session with new credentials */
	void RefreshLogin();
	/** Logout of our current PlayFabParty session */
	void Logout();

	/** Handle the title preparing to suspend */
	void HandleAppSuspended();
	/** Handle the title preparing to resume from suspension */
	void HandleAppResume();
	/** Handle the title's network initialization changing */
	void HandleNetworkInitializationChanged(bool bNetworkInitializationStatus);

	/** Handle QOS data being requested from the OSS */
	void HandleQOSDataRequested(FName SessionName);

	/** Handle ticking PlayFabParty */
	bool Tick(float DeltaSeconds);

	/** Process the PlayFabParty event loop */
	void ProcessPlayFabPartyEvents();
	void HandleRegionsChanged(const Party::PartyRegionsChangedStateChange* const StateChange);
	void HandleConnectToNetworkCompleted(const Party::PartyConnectToNetworkCompletedStateChange* const StateChange);
	void HandleAuthenticateLocalUserCompleted(const Party::PartyAuthenticateLocalUserCompletedStateChange* const StateChange);
	void HandleCreateEndpointCompleted(const Party::PartyCreateEndpointCompletedStateChange* const StateChange);
	void HandleEndpointDestroyed(const Party::PartyEndpointDestroyedStateChange* const StateChange);
	void HandleNetworkDestroyed(const Party::PartyNetworkDestroyedStateChange* const StateChange);
	void HandleEndpointMessageReceived(const Party::PartyEndpointMessageReceivedStateChange* const StateChange);
	void HandleCreateInvitationCompleted(const Party::PartyCreateInvitationCompletedStateChange* const StateChange);
	void HandleLeaveNetworkCompleted(const Party::PartyLeaveNetworkCompletedStateChange* StateChange);

	/** Process the PlayFabParty Xbox event loop */
	void ProcessPlayFabPartyXboxEvents();
	void HandleXboxCreateLocalChatUserCompleted(const Party::PartyXblCreateLocalChatUserCompletedStateChange* const StateChange);
	void HandleLoginToPlayFabCompleted(const Party::PartyXblLoginToPlayFabCompletedStateChange* const StateChange);
	void HandleGetEntityIdsFromXboxLiveUserIdsCompleted(const Party::PartyXblGetEntityIdsFromXboxLiveUserIdsCompletedStateChange* const StateChange);

protected:
#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE
	void HandleLocalPlayerCreationWithEntityToken(const Party::PartyXblLoginToPlayFabCompletedStateChange* const);
#else
	void HandleLocalPlayerCreationWithEntityHandle();
#endif
		
	/** Reference to the socket subsystem that created us */
	FPlayFabPartySocketSubsystem& SocketSubsystem;
	/** The AppId for this application from PlayFab's website */
	FString AppId;
	/** The URL for this application PlayFab's website */
	FString PlayFabURL;
	/** The delay in seconds after a login completes (success or failure) before another login may occur */
	double LoginFailureDelaySeconds = 15.0;

#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE
	/** The delay in seconds after a successful login complete before we should login again to get a new token */
	double LoginRefreshDelaySeconds = 7200.0;
#endif

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
	/** Is PlayFabCore currently initialized? */
	std::atomic<bool> bIsPlayFabCoreInit = false;
#endif
	/** Is PlayFabParty currently initialized? */
	std::atomic<bool> bIsPlayFabPartyInit = false;
	/** Is PlayFabParty Xbox currently initialized? */
	std::atomic<bool> bIsPlayFabPartyXboxInit = false;

	/** Is the app currently suspended? */
	bool bIsAppSuspending = false;
	/** Is the network ready for usage? */
	bool bIsNetworkReady = false;

	/** The flag to indicate if the game is connected playfab network. Since it's async operation, we can't create another socket to connect to playfab network, while previous socket hasn't finished leaving playfab network yet */
	bool bIsNetworkConnected = false;

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
	/** Handle to manage PlayFabCore lifecycle */
	std::atomic<PFServiceConfigHandle> PFCoreServiceHandle {};
#endif
	
	/** Handle to our QOS Data binding */
	FDelegateHandle QOSDataRequestedHandle;

	/** Handle to our tick function binding */
	FTSTicker::FDelegateHandle TickHandle;
	/** Handle to our application suspending binding */
	FDelegateHandle AppSuspendingHandle;
	/** Handle to our application resuming binding */
	FDelegateHandle AppResumingHandle;
	/** Handle to our network initialization changed binding */
	FDelegateHandle NetworkInitializationChangedHandle;

	/** The user we are logged in as (if non-zero) */
	uint64 LocalUserXuid = 0;
	/** The local users's login state for deciding if we want to login or not */
	EPlayFabPartyLoginState LocalUserLoginState = EPlayFabPartyLoginState::NeedLogin;
	/** The next time we can possibly try logging in again (throttle protection for login requests) */
	double NextLoginTimeSeconds = 0.0;

#if _GRDK_EDITION < 251000 || WITH_LEGACY_GDK_FOLDER_STRUCTURE // Token refresh is handled automatically after 251000
	/** The next time we should refresh our login credentials (to ensure they do not expire while in use) */
	double NextLoginRefreshTimeSeconds = 0.0;
#endif

	/** Our local xbox chat user */
	Party::PartyXblLocalChatUser* PartyXboxLocalChatUser = nullptr;
	/** Our local user object (only set if we are successfully logged in) */
	Party::PartyLocalUser* PartyLocalUser = nullptr;

	/** Cached QOS ping latency for Azure datacenters (in JSON format) for XboxLive sessions */
	TOptional<FString> QOSData;

	/** Map of party networks to the socket contexts that own the network */
	TMap<Party::PartyNetwork*, uint64> PartyToSocketContextMap;

	/** Hold the PlayFab socket for the listen server, to make sure not create it again(to keep the same listen server url) even if using non-seamless travel */
	TSharedPtr<FSocket> PlayFabSocketListenServer;

	/** Map of Xbox XUIDs to PlayFab Entity IDs */
	TMap<uint64, TUniqueObj<FPlayFabPartyEntityId>> XboxUserIdToPlayFabEntityIdMap;

	/** Map of Xbox XUIDs to PlayFab Entity Tokens */
	TMap<uint64, TUniqueObj<FPlayFabPartyEntityToken>> XboxUserIdToPlayFabEntityTokenMap;

#if _GRDK_EDITION >= 251000 && !WITH_LEGACY_GDK_FOLDER_STRUCTURE
	/** Map of Xbox XUIDs to PlayFab Entity Handles */
	TMap<uint64, PFEntityHandle> XboxUserIdToPlayFabEntityHandleMap;
#endif
};
#endif // WITH_PLAYFAB_PARTY
