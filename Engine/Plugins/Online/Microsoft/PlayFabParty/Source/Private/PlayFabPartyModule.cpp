// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayFabPartyModule.h"
#include "Modules/ModuleManager.h"
#if WITH_PLAYFAB_PARTY
#include "PlayFabPartySocketSubsystem.h"
#include "PlayFabPartyLog.h"
#include "PlayFabParty.h"
#include "PlayFabPartyLive.h"

#include "SocketSubsystemModule.h"
#include "UObject/NameTypes.h"
#include "OnlineSubsystem.h"
#endif // WITH_PLAYFAB_PARTY


#if WITH_PLAYFAB_PARTY
namespace
{
	static void* PARTY_CALLBACK UE_PlayFabPartyAllocate(size_t Size, uint32_t MemoryTypeId)
	{
		return FMemory::Malloc(Size);
	}

	static void PARTY_CALLBACK UE_PlayFabPartyFree(void* Pointer, uint32_t MemoryTypeId)
	{
		FMemory::Free(Pointer);
	}
}
#endif // WITH_PLAYFAB_PARTY


void FPlayFabPartyModule::StartupModule()
{
#if WITH_PLAYFAB_PARTY
	UE_LOGF(LogPlayFabParty, Log, "FPlayFabPartyModule::StartupModule()");

	// Setup our memory allocator once so PlayFab allocations are tracked
	static const bool bSetCallbacks = []() -> bool
	{
		Party::PartyManager::GetSingleton().SetMemoryCallbacks(UE_PlayFabPartyAllocate, UE_PlayFabPartyFree);
		Party::PartyXblManager::GetSingleton().SetMemoryCallbacks(UE_PlayFabPartyAllocate, UE_PlayFabPartyFree);
		return true;
	}();

	FModuleManager::LoadModuleChecked<FOnlineSubsystemModule>("OnlineSubsystem");
	FSocketSubsystemModule& SocketSubsystem = FModuleManager::LoadModuleChecked<FSocketSubsystemModule>("Sockets");

	check(!PlayFabPartySocketSubsystem.IsValid());
	PlayFabPartySocketSubsystem = MakeUnique<FPlayFabPartySocketSubsystem>();

	FString ErrorMessage;
	if (!PlayFabPartySocketSubsystem->Init(ErrorMessage))
	{
		UE_CLOGF(!IsRunningCommandlet(), LogPlayFabParty, Warning, "Failed to initialize: ErrorMessage=[%ls]", *ErrorMessage);
		PlayFabPartySocketSubsystem.Reset();
		return;
	}

	SocketSubsystem.RegisterSocketSubsystem(PLAYFABPARTY_SOCKETSUBSYSTEM, PlayFabPartySocketSubsystem.Get(), false);
#endif // WITH_PLAYFAB_PARTY
}

void FPlayFabPartyModule::ShutdownModule()
{
#if WITH_PLAYFAB_PARTY
	UE_LOGF(LogPlayFabParty, Log, "FPlayFabPartyModule::ShutdownModule()");

	if (PlayFabPartySocketSubsystem.IsValid())
	{
		FSocketSubsystemModule& SocketSubsystem = FModuleManager::LoadModuleChecked<FSocketSubsystemModule>("Sockets");
		SocketSubsystem.UnregisterSocketSubsystem(PLAYFABPARTY_SOCKETSUBSYSTEM);

		PlayFabPartySocketSubsystem.Reset();
	}
#endif // WITH_PLAYFAB_PARTY
}

bool FPlayFabPartyModule::SupportsDynamicReloading()
{
	return false;
}

bool FPlayFabPartyModule::SupportsAutomaticShutdown()
{
	// Shutdown gets called by the SocketSubsystem, if we were registered (and we don't do anything if we weren't)
	return false;
}

IMPLEMENT_MODULE(FPlayFabPartyModule, PlayFabParty);
