// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


#if WITH_PLAYFAB_PARTY
class FPlayFabPartySocketSubsystem;
#endif

class FPlayFabPartyModule
	: public IModuleInterface
{
public:
	//~ Begin IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	virtual bool SupportsDynamicReloading() override;
	virtual bool SupportsAutomaticShutdown() override;
	//~ End IModuleInterface Interface

	/**
	 * Singleton-like access to the PlayFab module instance
	 * @return Returns FPlayFabPartyModule singleton instance, loading the module on demand if needed
	 */
	static inline FPlayFabPartyModule& Get()
	{
		return FModuleManager::LoadModuleChecked<FPlayFabPartyModule>("PlayFabParty");
	}

#if WITH_PLAYFAB_PARTY
	/** Instance of our SocketSubsystem */
	TUniquePtr<FPlayFabPartySocketSubsystem> PlayFabPartySocketSubsystem;
#endif
};
