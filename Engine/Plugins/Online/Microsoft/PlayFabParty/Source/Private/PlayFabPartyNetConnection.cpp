// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayFabPartyNetConnection.h"
#if WITH_GRDK
#include "PlayFabPartyNetDriver.h"
#include "PlayFabPartyInternetAddr.h"
#include "PlayFabPartySocket.h"
#endif

UPlayFabPartyNetConnection::UPlayFabPartyNetConnection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPlayFabPartyNetConnection::InitLocalConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	DisableAddressResolution();

	Super::InitLocalConnection(InDriver, InSocket, InURL, InState, InMaxPacket, InPacketOverhead);
}

void UPlayFabPartyNetConnection::InitRemoteConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, const FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket, int32 InPacketOverhead)
{
	DisableAddressResolution();

	Super::InitRemoteConnection(InDriver, InSocket, InURL, InRemoteAddr, InState, InMaxPacket, InPacketOverhead);
}
