// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IpConnection.h"
#include "PlayFabPartyNetConnection.generated.h"

//class FInternetAddr;

UCLASS(Transient, Config=Engine)
class UPlayFabPartyNetConnection
	: public UIpConnection
{
	GENERATED_BODY()

public:
	explicit UPlayFabPartyNetConnection(const FObjectInitializer& ObjectInitializer);

	//~ Begin NetConnection Interface
	virtual void InitLocalConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	virtual void InitRemoteConnection(UNetDriver* InDriver, FSocket* InSocket, const FURL& InURL, const FInternetAddr& InRemoteAddr, EConnectionState InState, int32 InMaxPacket = 0, int32 InPacketOverhead = 0) override;
	//~ End NetConnection Interface
};
