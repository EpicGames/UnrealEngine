// Copyright Epic Games, Inc. All Rights Reserved.

#include "PlayFabPartyNetDriver.h"
#if WITH_PLAYFAB_PARTY
#include "PlayFabParty.h"
#include "PlayFabPartySocketSubsystem.h"
#include "PlayFabPartyNetConnection.h"
#include "PlayFabPartyInternetAddr.h"
#include "PlayFabPartySocket.h"
#endif // WITH_PLAYFAB_PARTY
#include "PlayFabPartyLog.h"

#include "OnlineSubsystem.h"
#include "OnlineSubsystemUtils.h"
#include "Sockets.h"

bool UPlayFabPartyNetDriver::IsAvailable() const
{
#if WITH_PLAYFAB_PARTY
	if (!Online::GetSubsystem(FindWorld(), GDK_SUBSYSTEM))
	{
		return false;
	}

	const FPlayFabPartySocketSubsystem* PlayFabPartySocketSubsystem = GetPlayFabSocketSubsystem();
	if (!PlayFabPartySocketSubsystem)
	{
		return false;
	}

	return PlayFabPartySocketSubsystem->IsPlayFabPartyInitialized();
#else
	return false;
#endif // WITH_PLAYFAB_PARTY
}

bool UPlayFabPartyNetDriver::InitSocket(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, bool bReuseSocket, FString& Error)
{
#if WITH_PLAYFAB_PARTY
	if (!IsAvailable())
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failing request as network is currently unavailable.");
		return false;
	}

	if (!UNetDriver::InitBase(bInitAsClient, InNotify, URL, bReuseAddressAndPort, Error))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to init driver base");
		return false;
	}

	// Determine what session name to use for this PlayFab socket
	FName SessionName = NAME_GameSession;
	if (const TCHAR* SessionNameString = URL.GetOption(TEXT("SessionName"), nullptr))
	{
		SessionName = FName(SessionNameString);
	}

	FPlayFabPartySocketSubsystem* const SocketSubsystem = GetPlayFabSocketSubsystem();
	check(SocketSubsystem);

	// Create our socket
	SetSocketAndLocalAddress(SocketSubsystem->CreatePlayFabSocket(*Online::GetSubsystem(FindWorld(), GDK_SUBSYSTEM), SessionName, bReuseSocket));
	if (GetSocket() == nullptr)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to create socket");
		return false;
	}

	return true;
#else
	return false;
#endif // WITH_PLAYFAB_PARTY
}

bool UPlayFabPartyNetDriver::InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error)
{
#if WITH_PLAYFAB_PARTY
	UE_LOGF(LogPlayFabParty, Verbose, "Connecting using PlayFabPartyNetDriver. ConnectUrl=[%ls]", *ConnectURL.ToString());

	bool bReuseSocket = false;
	if (!InitSocket(true, InNotify, ConnectURL, false, bReuseSocket, Error))
	{
		return false;
	}

	// Validate Network Address
	bool bIsValid = false;
	TSharedRef<FPlayFabPartyInternetAddr> PlayFabNetwork = MakeShared<FPlayFabPartyInternetAddr>();
	PlayFabNetwork->SetIp(*ConnectURL.Host, bIsValid);
	if (!bIsValid)
	{
		Error = TEXT("Invalid PlayFabParty Network Descriptor");
		UE_LOGF(LogPlayFabParty, Warning, "Invalid PlayFabParty Network Descriptor");
		return false;
	}

	// Validate Port
	if (ConnectURL.Port < 0)
	{
		Error = TEXT("Invalid Network Port");
		UE_LOGF(LogPlayFabParty, Warning, "Invalid Network Port");
		return false;
	}
	PlayFabNetwork->SetPort(ConnectURL.Port);

	// Reference to our newly created socket
	FPlayFabPartySocket* PlayFabSocket = GetPlayFabSocket();
	check(PlayFabSocket);

	// Create an unreal connection to the server
	UPlayFabPartyNetConnection* Connection = NewObject<UPlayFabPartyNetConnection>(NetConnectionClass);
	check(Connection);

	// Set it as the server connection before anything else so everything knows this is a client
	ServerConnection = Connection;
	Connection->InitLocalConnection(this, PlayFabSocket, ConnectURL, EConnectionState::USOCK_Pending);

	CreateInitialClientChannels();

	return PlayFabSocket->Connect(PlayFabNetwork.Get());
#else
	return false;
#endif // WITH_PLAYFAB_PARTY
}

bool UPlayFabPartyNetDriver::InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error)
{
#if WITH_PLAYFAB_PARTY
	UE_LOGF(LogPlayFabParty, Verbose, "Using PlayFabPartyNetDriver listen server. LocalURL=[%ls]", *LocalURL.ToString());

	bool bReuseSocket = true;
	if (!InitSocket(false, InNotify, LocalURL, bReuseAddressAndPort, bReuseSocket, Error))
	{
		return false;
	}

	FPlayFabPartySocketSubsystem* PlayFabSocketSubsystem = GetPlayFabSocketSubsystem();
	check(PlayFabSocketSubsystem);

	TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> PlayFabManager = PlayFabSocketSubsystem->GetPlayFabPartyManager();
	check(PlayFabManager.IsValid());

	FPlayFabPartySocket* PlayFabSocket = GetPlayFabSocket();
	check(PlayFabSocket);

	const FString InitialInvitationString = FGuid::NewGuid().ToString();

	int32 NumPlayers = Party::c_maxNetworkConfigurationMaxDeviceCount;
	if (const TCHAR* NumPlayersString = LocalURL.GetOption(TEXT("NumPlayers"), nullptr))
	{
		NumPlayers = FCString::Atoi(NumPlayersString);
		if (NumPlayers <= 0)
		{
			UE_LOGF(LogPlayFabParty, Error, "Attempted to create a match with no players. NumPlayers=[%d]", NumPlayers);

			Error = FString::Printf(TEXT("Invalid NumPlayers, was %d but must be between 1 and %u"), NumPlayers, Party::c_maxNetworkConfigurationMaxDeviceCount);
			return false;
		}
		else if (static_cast<uint32>(NumPlayers) > Party::c_maxNetworkConfigurationMaxDeviceCount)
		{
			UE_LOGF(LogPlayFabParty, Error, "Attempted to create a match with more than the maximum supported players. NumPlayers=[%d] MaxNumPlayers=[%u]", NumPlayers, Party::c_maxNetworkConfigurationMaxDeviceCount);

			Error = FString::Printf(TEXT("Invalid NumPlayers, was %d but must be between 1 and %u"), NumPlayers, Party::c_maxNetworkConfigurationMaxDeviceCount);
			return false;
		}
	}

	void* SocketContext = reinterpret_cast<void*>(PlayFabSocket->GetContext());

	TUniquePtr<Party::PartyNetworkDescriptor> PartyNetwork = PlayFabManager->CreateNetwork(InitialInvitationString, NumPlayers, SocketContext);
	if (!PartyNetwork.IsValid())
	{
		Error = TEXT("Failed to create PlayFabParty network");
		UE_LOGF(LogPlayFabParty, Warning, "Failed to create PlayFabParty network");
		return false;
	}

	StaticCastSharedPtr<FPlayFabPartyInternetAddr>(LocalAddr)->SetNetworkDescriptor(MoveTemp(PartyNetwork));
	if (!LocalAddr->IsValid())
	{
		Error = TEXT("Failed to create network");
		UE_LOGF(LogPlayFabParty, Warning, "Created PlayFabParty LocalAddress was invalid");
		return false;
	}

	// Bind our specified port if provided
	if (!PlayFabSocket->Bind(*LocalAddr))
	{
		Error = TEXT("Could not bind local port");
		UE_LOGF(LogPlayFabParty, Warning, "Could not bind local port");
		return false;
	}

	if (!PlayFabSocket->Listen(0))
	{
		Error = TEXT("Could not listen");
		UE_LOGF(LogPlayFabParty, Warning, "Could not listen on socket");
		return false;
	}

	// Save our invite string for when we connect to the network
	PlayFabSocket->AuthenticateWithNetwork(InitialInvitationString);

	InitConnectionlessHandler();

	UE_LOGF(LogPlayFabParty, Verbose, "Initialized as a PlayFabParty listen server");
	return true;
#else
	UE_LOGF(LogPlayFabParty, Verbose, "GRDK is not available");
	return false;
#endif // WITH_PLAYFAB_PARTY 
}

ISocketSubsystem* UPlayFabPartyNetDriver::GetSocketSubsystem()
{
#if WITH_PLAYFAB_PARTY
	return ISocketSubsystem::Get(PLAYFABPARTY_SOCKETSUBSYSTEM);
#else
	return nullptr;
#endif // WITH_PLAYFAB_PARTY
}

bool UPlayFabPartyNetDriver::IsNetResourceValid()
{
#if WITH_PLAYFAB_PARTY
	FPlayFabPartySocket* PlayFabSocket = GetPlayFabSocket();
	if (!PlayFabSocket)
	{
		return false;
	}

	return PlayFabSocket->GetConnectionState() == ESocketConnectionState::SCS_Connected;
#else
	return false;
#endif // WITH_PLAYFAB_PARTY
}

void UPlayFabPartyNetDriver::TickDispatch(float DeltaTime)
{
	Super::TickDispatch(DeltaTime);

#if WITH_PLAYFAB_PARTY
	FPlayFabPartySocket* PlayFabSocket = GetPlayFabSocket();
	if (PlayFabSocket)
	{
		PlayFabSocket->TickDispatch(DeltaTime);

		// Playfab socket could delay the initialization of connection, so error could happen when connect to Playfab after InitConnect. Make sure to close the connection if that happen.
		if (PlayFabSocket->GetConnectionState() == SCS_ConnectionError && ServerConnection->GetConnectionState() != USOCK_Closed)
		{
			ServerConnection->SetConnectionState(USOCK_Closed);
		}
	}
#endif // WITH_PLAYFAB_PARTY
}

void UPlayFabPartyNetDriver::Shutdown()
{
	UE_LOGF(LogPlayFabParty, Verbose, "Shutting down PlayFabParty NetDriver");

	Super::Shutdown();
}

#if WITH_PLAYFAB_PARTY
FPlayFabPartySocket* UPlayFabPartyNetDriver::GetPlayFabSocket()
{
	return static_cast<FPlayFabPartySocket*>(GetSocket());
}

FPlayFabPartySocketSubsystem* UPlayFabPartyNetDriver::GetPlayFabSocketSubsystem() const
{
	return static_cast<FPlayFabPartySocketSubsystem*>(ISocketSubsystem::Get(PLAYFABPARTY_SOCKETSUBSYSTEM));
}
#endif // WITH_PLAYFAB_PARTY

UWorld* UPlayFabPartyNetDriver::FindWorld() const
{
	UWorld* MyWorld = GetWorld();

	// If we don't have a world, we may be a pending net driver
	if (!MyWorld && GEngine)
	{
		if (FWorldContext* WorldContext = GEngine->GetWorldContextFromPendingNetGameNetDriver(this))
		{
			MyWorld = WorldContext->World();
		}
	}

	return MyWorld;
}
