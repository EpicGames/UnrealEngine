// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IpNetDriver.h"
#include "PlayFabPartyNetDriver.generated.h"

class ISocketSubsystem;
class FPlayFabPartySocket;
class FPlayFabPartySocketSubsystem;

UCLASS(Transient, Config=Engine)
class UPlayFabPartyNetDriver
	: public UIpNetDriver
{
	GENERATED_BODY()

public:
	//~ Begin UNetDriver Interface
	virtual bool IsAvailable() const override;
	virtual bool InitConnect(FNetworkNotify* InNotify, const FURL& ConnectURL, FString& Error) override;
	virtual bool InitListen(FNetworkNotify* InNotify, FURL& LocalURL, bool bReuseAddressAndPort, FString& Error) override;
	virtual ISocketSubsystem* GetSocketSubsystem() override;
	virtual bool IsNetResourceValid() override;
	virtual void TickDispatch(float DeltaTime) override;
	virtual void Shutdown() override;
	//~ End UNetDriver Interface

	bool InitSocket(bool bInitAsClient, FNetworkNotify* InNotify, const FURL& URL, bool bReuseAddressAndPort, bool bReuseSocket, FString& Error);

protected:
#if WITH_PLAYFAB_PARTY
	/** Helper to auto-cast and return our socket as a PlayFabParty socket */
	FPlayFabPartySocket* GetPlayFabSocket();

	/** Helper to auto-cast and return our socketsubsystem as a PlayFabParty socketsubsystem */
	FPlayFabPartySocketSubsystem* GetPlayFabSocketSubsystem() const;
#endif // WITH_PLAYFAB_PARTY

protected:
	/** Get the world this NetDriver belongs to */
	UWorld* FindWorld() const;
};
