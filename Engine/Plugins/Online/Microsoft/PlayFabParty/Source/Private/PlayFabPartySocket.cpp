// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_PLAYFAB_PARTY
#include "PlayFabPartySocket.h"
#include "PlayFabPartySocketSubsystem.h"
#include "PlayFabPartyInternetAddr.h"
#include "PlayFabParty.h"
#include "PlayFabPartyLive.h"
#include "PlayFabPartyLog.h"

#include "Online/OnlineSessionNames.h"
#include "OnlineSubsystem.h"
#include "OnlineSessionSettings.h"
#include "Online/OnlineSessionNames.h"
#include "HAL/IConsoleManager.h"

#define SETTING_PLAYFABPARTY_INVITATION TEXT("PFP_INVITE")

TAutoConsoleVariable<float> CVarPlayFabTimeToWaitForEvent(
	TEXT("Online.PlayFab.CVarPlayFabTimeToWaitForEvent"),
	15.0f,
	TEXT("The time to wait for event before set time out error."),
	ECVF_Default
);

FPlayFabPartySocket::FPlayFabPartySocket(FPlayFabPartySocketSubsystem& InSocketSubsystem, IOnlineSubsystem& InOnlineSubsystem, const FName InSessionName)
	: FSocket(SOCKTYPE_Streaming, FString(PLAYFABPARTY_SOCKETSUBSYSTEM), PLAYFABPARTY_SOCKETSUBSYSTEM)
	, SocketSubsystem(InSocketSubsystem)
	, WeakSessionInterface(InOnlineSubsystem.GetSessionInterface())
	, BoundSessionName(InSessionName)
	, SocketContext(SocketSubsystem.GenerateSocketContext(this))
	, TimeToWaitForLeaveNetworkComplete(CVarPlayFabTimeToWaitForEvent.GetValueOnGameThread())
	, TimeToWaitForNewInvitationData(CVarPlayFabTimeToWaitForEvent.GetValueOnGameThread())
{
	if (IOnlineSessionPtr SessionPtr = WeakSessionInterface.Pin())
	{
		CreateSessionCompleteHandle = SessionPtr->AddOnCreateSessionCompleteDelegate_Handle(FOnCreateSessionCompleteDelegate::CreateRaw(this, &FPlayFabPartySocket::HandleCreateSessionComplete));
		SessionCustomDataChangedHandle = SessionPtr->AddOnSessionSettingsUpdatedDelegate_Handle(FOnSessionSettingsUpdatedDelegate::CreateRaw(this, &FPlayFabPartySocket::HandleSessionCustomDataChanged));
		SessionParticipantJoinedHandle = SessionPtr->AddOnSessionParticipantJoinedDelegate_Handle(FOnSessionParticipantJoinedDelegate::CreateRaw(this, &FPlayFabPartySocket::HandleSessionParticipantJoined));
		SessionParticipantLeftHandle = SessionPtr->AddOnSessionParticipantLeftDelegate_Handle(FOnSessionParticipantLeftDelegate::CreateRaw(this, &FPlayFabPartySocket::HandleSessionParticipantLeft));
	}
}

FPlayFabPartySocket::~FPlayFabPartySocket()
{
	CloseImpl();

	SocketSubsystem.ReleaseSocketContext(SocketContext);

	if (IOnlineSessionPtr SessionPtr = WeakSessionInterface.Pin())
	{
		if (SessionParticipantJoinedHandle.IsValid())
		{
			SessionPtr->ClearOnSessionParticipantJoinedDelegate_Handle(SessionParticipantJoinedHandle);
			SessionParticipantJoinedHandle.Reset();
		}

		if (SessionParticipantLeftHandle.IsValid())
		{
			SessionPtr->ClearOnSessionParticipantLeftDelegate_Handle(SessionParticipantLeftHandle);
			SessionParticipantLeftHandle.Reset();
		}

		if (SessionCustomDataChangedHandle.IsValid())
		{
			SessionPtr->ClearOnSessionSettingsUpdatedDelegate_Handle(SessionCustomDataChangedHandle);
			SessionCustomDataChangedHandle.Reset();
		}

		if (CreateSessionCompleteHandle.IsValid())
		{
			SessionPtr->ClearOnCreateSessionCompleteDelegate_Handle(CreateSessionCompleteHandle);
			CreateSessionCompleteHandle.Reset();
		}
	}
}

bool FPlayFabPartySocket::Shutdown(ESocketShutdownMode Mode)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::Close()
{
	if (bIsServer)
	{
		// To avoid getting different URL each time, we never close socket for listen server
		return true;
	}

	return CloseImpl();
}

bool FPlayFabPartySocket::CloseImpl()
{
	if (!PartyNetwork)
	{
		check(PartyLocalEndpoint == nullptr);

		return true;
	}

	if (PartyLocalEndpoint)
	{
		PartyNetwork->DestroyEndpoint(PartyLocalEndpoint, nullptr);
		PartyLocalEndpoint = nullptr;

		bIsEndpointConnected = false;
	}

	bIsLocalUserAuthenticated = false;
	bExpectingLeaveNetworkComplete = false;
	bExpectingNewInvitationData = false;

	UE_LOGF(LogPlayFabParty, Log, "Leaving PlayFabParty Network");
	PartyNetwork->LeaveNetwork(nullptr);
	PartyNetwork = nullptr;
	bIsNetworkConnected = false;

	return true;
}

bool FPlayFabPartySocket::Bind(const FInternetAddr& Addr)
{
	if (NetworkAddress.IsValid())
	{
		if (bIsServer && NetworkAddress->CompareEndpoints(Addr))
		{
			// Could be reusing the same socket on listen server, so it's expected
			return true;
		}
		else
		{
			UE_LOGF(LogPlayFabParty, Error, "Attempted to bind to already bound socket, ignoring. SessionName=[%ls] CurrentAddress=[%ls] NewAddress=[%ls]", *BoundSessionName.ToString(), *NetworkAddress->ToString(false), *Addr.ToString(false));
			SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
			return false;
		}
	}

	return SetNetworkAddress(Addr);
}

bool FPlayFabPartySocket::Connect(const FInternetAddr& Addr)
{
	bIsServer = false;

	if (!Addr.IsValid())
	{
		UE_LOGF(LogPlayFabParty, Warning, "Cannot connect to network, network address is invalid. SessionName=[%ls] Address=[%ls]", *BoundSessionName.ToString(), *Addr.ToString(true));
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EFAULT);
		return false;
	}

	if (!SetNetworkAddress(Addr))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Cannot connect to network, failed to apply network address. SessionName=[%ls] Address=[%ls]", *BoundSessionName.ToString(), *Addr.ToString(true));
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EFAULT);
		return false;
	}

	UE_LOGF(LogPlayFabParty, Log, "Starting connection to PlayFabParty Network as a client. SessionName=[%ls] Address=[%ls]", *BoundSessionName.ToString(), *Addr.ToString(true));

	TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> PartyManager = SocketSubsystem.GetPlayFabPartyManager();
	if (PartyManager.IsValid() && PartyManager->IsNetworkConnected())
	{
		// Delay the connect
		bExpectingLeaveNetworkComplete = true;
		TimeToWaitForLeaveNetworkComplete = CVarPlayFabTimeToWaitForEvent.GetValueOnGameThread();
		return true;
	}

	return Connect_Internal();
}

bool FPlayFabPartySocket::Listen(int32 MaxBacklog)
{
	bIsServer = true;

	if (!NetworkAddress.IsValid())
	{
		UE_LOGF(LogPlayFabParty, Warning, "Cannot start listen server, network address has not been set. SessionName=[%ls]", *BoundSessionName.ToString());
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EFAULT);
		return false;
	}

	if (!NetworkAddress->IsValid())
	{
		UE_LOGF(LogPlayFabParty, Warning, "Cannot start listen server, network descriptor is invalid. SessionName=[%ls] Address=[%ls]", *BoundSessionName.ToString(), *NetworkAddress->ToString(true));
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EFAULT);
		return false;
	}

	UE_LOGF(LogPlayFabParty, Log, "Starting connection to PlayFabParty Network as a listen server. SessionName=[%ls] Address=[%ls]", *BoundSessionName.ToString(), *NetworkAddress->ToString(true));

	if (PartyNetwork)
	{
		// Could be reusing the same socket on listen server, so it's expected that PartyNetwork was created before
		return true;
	}

	return Connect_Internal();
}

bool FPlayFabPartySocket::WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::HasPendingData(uint32& PendingDataSize)
{
	const TUniquePtr<FPlayFabPartyPendingPacket>* PendingPacket = PacketQueue.Peek();
	if (!PendingPacket)
	{
		PendingDataSize = 0;
		return false;
	}

	if (!PendingPacket->IsValid())
	{
		PendingDataSize = 0;
		return false;
	}

	PendingDataSize = (*PendingPacket)->Data.Num();
	return true;
}

FSocket* FPlayFabPartySocket::Accept(const FString& InSocketDescription)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return nullptr;
}

FSocket* FPlayFabPartySocket::Accept(FInternetAddr& OutAddr, const FString& InSocketDescription)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return nullptr;
}

bool FPlayFabPartySocket::SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination)
{
	BytesSent = 0;

	const uint16 UniqueId = Destination.GetPort();
	if (UniqueId == 0)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Attempted to send packet to unknown endpoint");

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	if (GetConnectionState() != ESocketConnectionState::SCS_Connected)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Attempted to send packet before connection completed");

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_ENOTCONN);
		return false;
	}

	Party::PartyEndpoint* PartyEndpoint = nullptr;

	const PartyError FindEndpointError = PartyNetwork->FindEndpointByUniqueIdentifier(UniqueId, &PartyEndpoint);
	if (PARTY_FAILED(FindEndpointError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Unable to find endpoint for unique id. UniqueId=[%u]", static_cast<uint32>(UniqueId));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EHOSTUNREACH);
		return false;
	}

    const uint32_t TargetEndpointCount = 1;
	Party::PartyEndpoint* const TargetEndpoints[] = { PartyEndpoint };
    const Party::PartySendMessageOptions Options = Party::PartySendMessageOptions::BestEffortDelivery | Party::PartySendMessageOptions::NonsequentialDelivery | Party::PartySendMessageOptions::CopyDataBuffers | Party::PartySendMessageOptions::CoalesceOpportunistically;
    const Party::PartySendMessageQueuingConfiguration* MessageConfiguration = nullptr;
    const uint32_t DataBufferCount = 1;
	const Party::PartyDataBuffer DataBuffer[] = { {Data, Count} };
    void* AsyncMessageContext = nullptr;

	const PartyError SendMessageError = PartyLocalEndpoint->SendMessage(
		TargetEndpointCount,
		&TargetEndpoints[0],
		Options,
		MessageConfiguration,
		DataBufferCount,
		DataBuffer,
		AsyncMessageContext
	);
	if (PARTY_FAILED(SendMessageError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to send data to network. ErrorCode=[%u] Error=[%ls]", SendMessageError, *GetPlayFabPartyErrorMessage(SendMessageError));

		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EINVAL);
		return false;
	}

	BytesSent = Count;
	return true;
}

bool FPlayFabPartySocket::Send(const uint8* Data, int32 Count, int32& BytesSent)
{
	BytesSent = 0;

	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags)
{
	checkSlow(Data);
	checkSlow(BufferSize >= 0);

	BytesRead = 0;

	TUniquePtr<FPlayFabPartyPendingPacket> PacketStorage;
	const TUniquePtr<FPlayFabPartyPendingPacket>* PacketToRead = &PacketStorage;

	if (Flags == ESocketReceiveFlags::Peek)
	{
		PacketToRead = PacketQueue.Peek();
	}
	else if (Flags == ESocketReceiveFlags::None)
	{
		PacketQueue.Dequeue(PacketStorage);
	}
	else
	{
		// Not supported
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
		return false;
	}

	if (!PacketToRead || !PacketToRead->IsValid())
	{
		// No data available
		SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EWOULDBLOCK);
		return false;
	}

	const int32 BytesAvailable = (*PacketToRead)->Data.Num();
	const int32 BytesToWrite = FMath::Min(BytesAvailable, BufferSize);

	// Write data available
	FMemory::Memcpy(Data, (*PacketToRead)->Data.GetData(), BytesToWrite);
	BytesRead = BytesToWrite;

	// Write sender
	static_cast<FPlayFabPartyInternetAddr&>(Source) = (*PacketToRead)->Sender;

	return true;
}

bool FPlayFabPartySocket::Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags)
{
	BytesRead = 0;

	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime)
{
	return false;
}

ESocketConnectionState FPlayFabPartySocket::GetConnectionState()
{
	if (SocketConnectError.IsSet())
	{
		return ESocketConnectionState::SCS_ConnectionError;
	}

	if (!NetworkAddress.IsValid())
	{
		return ESocketConnectionState::SCS_NotConnected;
	}

	if (!bIsNetworkConnected || !PartyNetwork)
	{
		return ESocketConnectionState::SCS_NotConnected;
	}

	if (!bIsLocalUserAuthenticated)
	{
		return ESocketConnectionState::SCS_NotConnected;
	}

	if (!bIsEndpointConnected || !PartyLocalEndpoint)
	{
		return ESocketConnectionState::SCS_NotConnected;
	}

	return ESocketConnectionState::SCS_Connected;
}

void FPlayFabPartySocket::GetAddress(FInternetAddr& OutAddr)
{
	if (!NetworkAddress.IsValid())
	{
		static_cast<FPlayFabPartyInternetAddr&>(OutAddr) = FPlayFabPartyInternetAddr();
		return;
	}

	static_cast<FPlayFabPartyInternetAddr&>(OutAddr) = *NetworkAddress.Get();
}

bool FPlayFabPartySocket::GetPeerAddress(FInternetAddr& OutAddr)
{
	return false;
}

bool FPlayFabPartySocket::SetNonBlocking(bool bIsNonBlocking)
{
	/** Not supported, but we pretend we do */
	return true;
}

bool FPlayFabPartySocket::SetBroadcast(bool bAllowBroadcast)
{
	/** Not supported, but we pretend we do */
	return true;
}

bool FPlayFabPartySocket::SetNoDelay(bool bIsNoDelay)
{
	/** Not supported, but we pretend we do */
	return true;
}

bool FPlayFabPartySocket::JoinMulticastGroup(const FInternetAddr& GroupAddress)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::LeaveMulticastGroup(const FInternetAddr& GroupAddress)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::SetMulticastLoopback(bool bLoopback)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::SetMulticastTtl(uint8 TimeToLive)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::SetMulticastInterface(const FInternetAddr& InterfaceAddress)
{
	/** Not supported */
	SocketSubsystem.SetLastSocketError(ESocketErrors::SE_EOPNOTSUPP);
	return false;
}

bool FPlayFabPartySocket::SetReuseAddr(bool bAllowReuse)
{
	/** Not supported, but we pretend we do */
	return true;
}

bool FPlayFabPartySocket::SetLinger(bool bShouldLinger, int32 Timeout)
{
	/** Not supported, but we pretend we do */
	return true;
}

bool FPlayFabPartySocket::SetRecvErr(bool bUseErrorQueue)
{
	/** Not supported, but we pretend we do */
	return true;
}

bool FPlayFabPartySocket::SetSendBufferSize(int32 Size, int32& NewSize)
{
	/** Not supported, but we pretend we do */
	return true;
}

bool FPlayFabPartySocket::SetReceiveBufferSize(int32 Size, int32& NewSize)
{
	/** Not supported, but we pretend we do */
	return true;
}

int32 FPlayFabPartySocket::GetPortNo()
{
	if (!NetworkAddress.IsValid())
	{
		return -1;
	}

	return NetworkAddress->GetPort();
}

void FPlayFabPartySocket::TickDispatch(float DeltaTime)
{
	if (bIsServer)
	{
		return;
	}

	if (bExpectingLeaveNetworkComplete)
	{
		TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> PartyManager = SocketSubsystem.GetPlayFabPartyManager();
		if (PartyManager.IsValid() && !PartyManager->IsNetworkConnected())
		{
			bExpectingLeaveNetworkComplete = false;

			if (!Connect_Internal())
			{
				SocketConnectError = ESocketErrors::SE_EFAULT;
			}

			return;
		}

		TimeToWaitForLeaveNetworkComplete -= DeltaTime;
		if (TimeToWaitForLeaveNetworkComplete < 0.0f)
		{
			bExpectingLeaveNetworkComplete = false;

			UE_LOGF(LogPlayFabParty, Warning, "Time out to waiting for leave network complete event.");
			SocketConnectError = ESocketErrors::SE_ETIMEDOUT;

			return;
		}
	}

	if (bExpectingNewInvitationData)
	{
		TimeToWaitForNewInvitationData -= DeltaTime;
		if (TimeToWaitForNewInvitationData <= 0.0f)
		{
			UE_LOGF(LogPlayFabParty, Warning, "No new invitation data received after waiting, closing socket.");
			SocketConnectError = ESocketErrors::SE_ETIMEDOUT;

			bExpectingNewInvitationData = false;
			Close();
		}
	}
}

uint64 FPlayFabPartySocket::GetContext() const
{
	return SocketContext;
}

bool FPlayFabPartySocket::IsServer() const
{
	return bIsServer;
}

bool FPlayFabPartySocket::IsNetworkConnected() const
{
	return PartyNetwork && bIsNetworkConnected;
}

bool FPlayFabPartySocket::HasAuthenticated() const
{
	return bIsLocalUserAuthenticated;
}

bool FPlayFabPartySocket::IsEndpointConnected() const
{
	return PartyLocalEndpoint && bIsEndpointConnected;
}

bool FPlayFabPartySocket::AuthenticateWithNetwork(const FString& NetworkInvitation)
{
	UE_LOGF(LogPlayFabParty, Verbose, "AuthenticateWithNetwork ");

	PendingNetworkInvitation = NetworkInvitation;

	// If we're not connected to the network, wait to apply this invitation until we are
	if (!IsNetworkConnected())
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Waiting to authenticate with the network until we are connected to it. SessionName=[%ls] NetworkInvitation=[%ls]", *BoundSessionName.ToString(), *NetworkInvitation);
		return true;
	}

	Party::PartyLocalUser* LocalUser = GetLocalUser();
	if (!LocalUser)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start authenticating local user with network as the local user has gone away. SessionName=[%ls] NetworkInvitation=[%ls]", *BoundSessionName.ToString(), *NetworkInvitation);
		return false;
	}

	void* SocketContextPtr = reinterpret_cast<void*>(SocketContext);
	const PartyError AuthenticateUserError = PartyNetwork->AuthenticateLocalUser(LocalUser, TCHAR_TO_ANSI(*NetworkInvitation), SocketContextPtr);
	if (PARTY_FAILED(AuthenticateUserError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start authenticating local user with PlayFabParty. ErrorCode=[%u] Error=[%ls] SessionName=[%ls] NetworkInvitation=[%ls]", AuthenticateUserError, *GetPlayFabPartyErrorMessage(AuthenticateUserError), *BoundSessionName.ToString(), *NetworkInvitation);
		return false;
	}

	UE_LOGF(LogPlayFabParty, Log, "Attempting to authenticate with network. SessionName=[%ls] NetworkInvitation=[%ls]", *BoundSessionName.ToString(), *NetworkInvitation);
	return true;
}

bool FPlayFabPartySocket::CreateLocalEndPoint()
{
	if (!bIsNetworkConnected)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start creating local endpoint as we are not connected to a network");
		return false;
	}
	check(PartyNetwork);

	if (!bIsLocalUserAuthenticated)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start creating local endpoint as we are not authenticated with the network");
		return false;
	}

	Party::PartyLocalUser* LocalUser = GetLocalUser();
	uint32 PropertyCount = 0;
	const PartyString* PropertyNameList = nullptr;
	const Party::PartyDataBuffer* PropertyKeyList = nullptr;
	void* SocketContextPtr = reinterpret_cast<void*>(SocketContext);

	PartyError CreateEndpointError = PartyNetwork->CreateEndpoint(LocalUser, PropertyCount, PropertyNameList, PropertyKeyList, SocketContextPtr, &PartyLocalEndpoint);
	if (PARTY_FAILED(CreateEndpointError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start creating local endpoint. ErrorCode=[%u] Error=[%ls]", CreateEndpointError, *GetPlayFabPartyErrorMessage(CreateEndpointError));
		return false;
	}

	UE_LOGF(LogPlayFabParty, Log, "Creating Local Endpoint. SessionName=[%ls]", *BoundSessionName.ToString());
	return true;
}

bool FPlayFabPartySocket::CreateInvitation()
{
	UE_LOGF(LogPlayFabParty, Verbose, "Creating party invitation");
	Party::PartyLocalUser* LocalUser = GetLocalUser();
	if (!LocalUser)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start creating party invitation, local user is invalid");

		return false;
	}

	TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> PartyManager = SocketSubsystem.GetPlayFabPartyManager();
	if (!PartyManager.IsValid())
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start creating party invitation, party manager is invalid");

		return false;
	}

	// Check if we have all of our member entity IDs
	TArray<uint64> MissingMembers;
	for (const uint64 MemberXuid : SessionMembers)
	{
		if (!PartyManager->HaveEntityIdForXuid(MemberXuid))
		{
			MissingMembers.Add(MemberXuid);
		}
	}

	// If we are missing any members, go query them now
	if (MissingMembers.Num() > 0)
	{
		const Party::PartyXblLocalChatUser* XboxChatUser = PartyManager->GetPartyXboxLocalChatUser();
		if (!XboxChatUser)
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to start creating party invitation, local xbox chat user invalid");

			Close();
			return false;
		}

		const uint32 MembersNum = MissingMembers.Num();
		const uint64* Members = MissingMembers.GetData();
		void* SocketContextPtr = reinterpret_cast<void*>(SocketContext);

		const PartyError GetEntityIdsError = Party::PartyXblManager::GetSingleton().GetEntityIdsFromXboxLiveUserIds(MembersNum, Members, XboxChatUser, SocketContextPtr);
		if (PARTY_FAILED(GetEntityIdsError))
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to start querying EntityIds. ErrorCode=[%u] Error=[%ls]", GetEntityIdsError, *GetPlayFabPartyErrorMessage(GetEntityIdsError));
			return false;
		}

		return true;
	}

	if (PartyInvitation)
	{
		PartyNetwork->RevokeInvitation(LocalUser, PartyInvitation, nullptr);
		PartyInvitation = nullptr;
	}

	// Create invitation string
	const FString InvitationString(FGuid::NewGuid().ToString());
	const auto ConvertedInitialInvitationString = StringCast<ANSICHAR>(*InvitationString);

	// Build array of members
	TArray<PartyString> EntityIdList;
	EntityIdList.AddUninitialized(SessionMembers.Num());
	for (int32 Index = 0; Index < EntityIdList.Num(); ++Index)
	{
		EntityIdList[Index] = PartyManager->GetEntityIdForXuid(SessionMembers[Index]);
	}

	Party::PartyInvitationConfiguration InvitationConfiguration = {};
	InvitationConfiguration.identifier = ConvertedInitialInvitationString.Get();
	InvitationConfiguration.revocability = Party::PartyInvitationRevocability::Creator;
	InvitationConfiguration.entityIdCount = EntityIdList.Num();
	InvitationConfiguration.entityIds = EntityIdList.GetData();

	void* SocketContextPtr = reinterpret_cast<void*>(SocketContext);

	Party::PartyInvitation* NewInvite = nullptr;

	const PartyError CreateInvitationError = PartyNetwork->CreateInvitation(LocalUser, &InvitationConfiguration, SocketContextPtr, &NewInvite);
	if (PARTY_FAILED(CreateInvitationError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to start creating party invitation. ErrorCode=[%u] Error=[%ls]", CreateInvitationError, *GetPlayFabPartyErrorMessage(CreateInvitationError));
		return false;
	}

	PartyInvitation = NewInvite;

	UE_LOGF(LogPlayFabParty, Log, "Started creating new party invitation for session. SessionName=[%ls]", *BoundSessionName.ToString());
	return true;
}

void FPlayFabPartySocket::HandleConnectToNetworkComplete()
{
	UE_LOGF(LogPlayFabParty, Log, "Successfully connected to network");

	bIsNetworkConnected = true;

	// Authenticate with our invitation now that we're connected
	if (PendingNetworkInvitation.IsSet())
	{
		const FString NetworkInvitation = MoveTemp(PendingNetworkInvitation.GetValue());
		PendingNetworkInvitation.Reset();

		AuthenticateWithNetwork(NetworkInvitation);
	}
	else if (!bIsServer)
	{
		// Check the session for an existing invitation that was created before our socket was created, but after we joined
		IOnlineSessionPtr SessionInterface = WeakSessionInterface.Pin();
		if (!SessionInterface.IsValid())
		{
			UE_LOGF(LogPlayFabParty, Warning, "Session Interface went away! Session=[%ls]", *BoundSessionName.ToString());
			return;
		}

		FOnlineSessionSettings* SessionSettings = SessionInterface->GetSessionSettings(BoundSessionName);
		if (!SessionSettings)
		{
			UE_LOGF(LogPlayFabParty, Warning, "Could not retrieve session settings! Session=[%ls]", *BoundSessionName.ToString());
			return;
		}

		// Attempt to extract our invitation
		FString InvitationKey;
		if (!SessionSettings->Get(SETTING_CUSTOM_JOIN_INFO, InvitationKey))
		{
			UE_LOGF(LogPlayFabParty, Verbose, "No invitation set yet, waiting for a SessionCustomDataChanged event instead. Session=[%ls]", *BoundSessionName.ToString());
			return;
		}
		SessionSettings->Get(SETTING_CUSTOM_JOIN_INFO, InvitationKey);
		const auto KeyStringConv = StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
		const char* const KeyPtr = reinterpret_cast<const char*>(KeyStringConv.Get());
		int32 Pos = InvitationKey.Find(KeyPtr);
		if (Pos > 0)
		{
			InvitationKey = InvitationKey.Mid(Pos + KeyStringConv.Length());
		}

		// Ensure our key is valid-ish
		if (InvitationKey.IsEmpty())
		{
			UE_LOGF(LogPlayFabParty, Verbose, "Empty invitation string set, waiting for a SessionCustomDataChanged event instead. Session=[%ls]", *BoundSessionName.ToString());
			return;
		}

		AuthenticateWithNetwork(InvitationKey);
	}
}

void FPlayFabPartySocket::HandleLocalUserAuthenticationComplete()
{
	UE_LOGF(LogPlayFabParty, Log, "Successfully authenticated with network");

	bIsLocalUserAuthenticated = true;

	CreateLocalEndPoint();
}

void FPlayFabPartySocket::HandleCreateLocalEndpointComplete()
{
	UE_LOGF(LogPlayFabParty, Log, "Successfully created local endpoint");

	bIsEndpointConnected = true;

	if (!PartyLocalEndpoint)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Ignoring new local endpoint as we have closed our socket since we started creating it. SessionName=[%ls]", *BoundSessionName.ToString());

		Close();
		return;
	}

	uint16 LocalUniqueId = 0u;
	const PartyError GetUniqueIdError = PartyLocalEndpoint->GetUniqueIdentifier(&LocalUniqueId);
	if (PARTY_FAILED(GetUniqueIdError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to get local network endpoint unique id. ErrorCode=[%u] Error=[%ls]", GetUniqueIdError, *GetPlayFabPartyErrorMessage(GetUniqueIdError));

		Close();
		return;
	}

	UE_LOGF(LogPlayFabParty, Log, "Local endpoint unique id is %u", static_cast<uint32>(LocalUniqueId));

	if (bIsServer)
	{
		// Pretend we just created the session if it already exists, now that our endpoint is valid
		IOnlineSessionPtr SessionInterface = WeakSessionInterface.Pin();
		if (!SessionInterface.IsValid())
		{
			UE_LOGF(LogPlayFabParty, Warning, "Could not update session with network information, our session interface went away! Session=[%ls]", *BoundSessionName.ToString());
			return;
		}

		if (SessionInterface->GetNamedSession(BoundSessionName))
		{
			HandleCreateSessionComplete(BoundSessionName, true);
		}
	}
}

void FPlayFabPartySocket::HandleEndpointMessageReceived(FPlayFabPartyPendingPacket&& Message)
{
	PacketQueue.Enqueue(MakeUnique<FPlayFabPartyPendingPacket>(MoveTemp(Message)));
}

void FPlayFabPartySocket::HandleCreateInvitationCompleted(Party::PartyInvitation* Invitation, const bool bWasSuccessful)
{
	// We have a newer invitation in process, ignore this one
	if (Invitation != PartyInvitation)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Non-latest party invitation finished creating, ignoring");
		return;
	}

	if (!bWasSuccessful)
	{
		UE_LOGF(LogPlayFabParty, Error, "Failed to create PlayFabParty invitation");

		PartyInvitation = nullptr;
		return;
	}

	const Party::PartyInvitationConfiguration* PartyInvitationConfig = nullptr;

	const PartyError GetInvitationConfigError = Invitation->GetInvitationConfiguration(&PartyInvitationConfig);
	if (PARTY_FAILED(GetInvitationConfigError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to get newly created party invitation's config. ErrorCode=[%u] Error=[%ls]", GetInvitationConfigError, *GetPlayFabPartyErrorMessage(GetInvitationConfigError));
		return;
	}
	check(PartyInvitationConfig);

	IOnlineSessionPtr SessionInterface = WeakSessionInterface.Pin();
	if (!SessionInterface.IsValid())
	{
		UE_LOGF(LogPlayFabParty, Warning, "Could not update session with network invitation, our session interface went away! Session=[%ls]", *BoundSessionName.ToString());

		Close();
		return;
	}

	FOnlineSessionSettings* SessionSettings = SessionInterface->GetSessionSettings(BoundSessionName);
	if (!SessionSettings)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Could not retrieve session settings, abandoning session! Session=[%ls]", *BoundSessionName.ToString());

		Close();
		return;
	}

	FString InviteValue(ANSI_TO_TCHAR(PartyInvitationConfig->identifier));

	SessionSettings->Set(SETTING_PLAYFABPARTY_INVITATION, MoveTemp(InviteValue), EOnlineDataAdvertisementType::ViaOnlineService);
	UE_LOGF(LogPlayFabParty, Verbose, "updating session with new PlayFabParty invitation string");

	if (!SessionInterface->UpdateSession(BoundSessionName, *SessionSettings, true))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to update session with new PlayFabParty invitation string");
		return;
	}
}

void FPlayFabPartySocket::HandleGetEntityIdsFromXboxLiveUserIdsCompleted(const bool bWasSuccessful)
{
	UE_LOGF(LogPlayFabParty, Log, "Finished querying EntityIDs for session members. bWasSuccessful=[%d]", bWasSuccessful);

	CreateInvitation();
}

Party::PartyLocalUser* FPlayFabPartySocket::GetLocalUser()
{
	TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> PlayFabManager = SocketSubsystem.GetPlayFabPartyManager();
	if (!ensure(PlayFabManager.IsValid()))
	{
		// This shouldn't happen, so logging it in case it does
		UE_LOGF(LogPlayFabParty, Warning, "Failed to get local user as our manager has shutdown");
		return nullptr;
	}

	return PlayFabManager->GetPartyLocalUser();
}

bool FPlayFabPartySocket::SetNetworkAddress(const FInternetAddr& Addr)
{
	if (!Addr.IsValid())
	{
		return false;
	}

	if (NetworkAddress.IsValid())
	{
		Close();
		NetworkAddress.Reset();
	}

	NetworkAddress = StaticCastSharedRef<FPlayFabPartyInternetAddr>(Addr.Clone());
	return true;
}

bool FPlayFabPartySocket::Connect_Internal()
{
	Party::PartyManager& PlayFabParty = Party::PartyManager::GetSingleton();

	void* SocketContextPtr = reinterpret_cast<void*>(SocketContext);

    const PartyError ConnectError = PlayFabParty.ConnectToNetwork(NetworkAddress->GetNetworkDescriptor(), SocketContextPtr, &PartyNetwork);
	if (PARTY_FAILED(ConnectError))
	{
		UE_LOGF(LogPlayFabParty, Warning, "Failed to connect to network due to error. ErrorCode=[%u] Error=[%ls]", ConnectError, *GetPlayFabPartyErrorMessage(ConnectError));

		return false;
	}

	return PartyNetwork != nullptr;
}

void FPlayFabPartySocket::HandleCreateSessionComplete(FName SessionName, bool bWasSuccessful)
{
	UE_LOGF(LogPlayFabParty, VeryVerbose, "Received CreateSessionComplete event. NewSession=[%ls] bWasSuccessful=[%d]", *SessionName.ToString(), bWasSuccessful);

	// We don't need to do anything if we're not the server
	if (!bIsServer)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Session create completed, but ignoring this event as we are a client. Session=[%ls]", *BoundSessionName.ToString());
		return;
	}

	// Not for this session
	if (BoundSessionName != SessionName)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring Session create for different session. BoundSession=[%ls] CreatedSession=[%ls]", *BoundSessionName.ToString(), *SessionName.ToString());
		return;
	}

	// Not successful
	if (!bWasSuccessful)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Create Session event completed, but failed. Waiting on success before we may advertise session. Session=[%ls]", *BoundSessionName.ToString());
		return;
	}

	// We're not fully connected yet, so we don't want to advertise our network before we can potentially talk to others
	if (!bIsEndpointConnected || !PartyLocalEndpoint)
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Session create completed, but our endpoint isn't ready yet. Waiting for local endpoint before we may advertise session. Session=[%ls]", *BoundSessionName.ToString());
		return;
	}

	IOnlineSessionPtr SessionInterface = WeakSessionInterface.Pin();
	if (!SessionInterface.IsValid())
	{
		UE_LOGF(LogPlayFabParty, Warning, "Could not update session with network invitation, our session interface went away! Session=[%ls]", *BoundSessionName.ToString());

		Close();
		return;
	}

	FOnlineSessionSettings* SessionSettings = SessionInterface->GetSessionSettings(BoundSessionName);
	if (!SessionSettings)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Could not retrieve session settings, abandoning session! Session=[%ls]", *BoundSessionName.ToString());

		Close();
		return;
	}

	if (!SessionSettings->Settings.Find(SETTING_CUSTOM_JOIN_INFO))
	{
		FString SessionKey(FString::Printf(TEXT("unreal://%s"),
			*FPlayFabPartyInternetAddr(*PartyLocalEndpoint).ToString(true)));
		SessionKey += StringCast<UTF8CHAR>(*(SETTING_CUSTOM_JOIN_INFO.ToString()));
		SessionKey += MoveTemp(PendingNetworkInvitation.GetValue());
		SessionSettings->Set(SETTING_CUSTOM_JOIN_INFO, MoveTemp(SessionKey), EOnlineDataAdvertisementType::ViaOnlineService);

		UE_LOGF(LogPlayFabParty, Verbose, "update session with playfab session key string");
		if (!SessionInterface->UpdateSession(BoundSessionName, *SessionSettings, true))
		{
			UE_LOGF(LogPlayFabParty, Warning, "Failed to update session with playfab session key string");

			Close();
		}
	}
}

void FPlayFabPartySocket::HandleSessionCustomDataChanged(FName SessionName, const FOnlineSessionSettings& SessionSettings)
{
	UE_LOGF(LogPlayFabParty, VeryVerbose, "Received SessionCustomDataChanged event. ChangedSession=[%ls]", *SessionName.ToString());

	// Host doesn't care about setting changes
	if (bIsServer)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring SessionCustomDataChanged event as we are the server. ChangedSession=[%ls]", *SessionName.ToString());
		return;
	}

	// Not for this session
	if (BoundSessionName != SessionName)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring SessionCustomDataChanged event for wrong session. BoundSessionName=[%ls] ChangedSession=[%ls]", *BoundSessionName.ToString(), *SessionName.ToString());
		return;
	}

	// We don't care about invites if we're already authenticated with this network
	if (bIsLocalUserAuthenticated)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring SessionCustomDataChanged event as we are already authenticated. BoundSessionName=[%ls] ChangedSession=[%ls]", *BoundSessionName.ToString(), *SessionName.ToString());
		return;
	}

	// Attempt to extract our invitation
	FString InvitationKey;
	if (!SessionSettings.Get(SETTING_PLAYFABPARTY_INVITATION, InvitationKey))
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received SessionCustomDataChanged event, but could not find a set invitation. ChangedSession=[%ls]", *SessionName.ToString());
		return;
	}

	// Ensure our key is valid-ish
	if (InvitationKey.IsEmpty())
	{
		UE_LOGF(LogPlayFabParty, Verbose, "Received SessionCustomDataChanged event, but the invitation was empty. ChangedSession=[%ls]", *SessionName.ToString());
		return;
	}

	// Ensure we either haven't tried any invitation or it's different from the last one we tried (we don't want to retry the same key, as this event fires for any custom data changed, not just invitations)
	if (!PendingNetworkInvitation.IsSet() || PendingNetworkInvitation.GetValue() != InvitationKey)
	{
		UE_LOGF(LogPlayFabParty, Log, "Received SessionCustomDataChanged event with new network invitation, attempting to authenticate with the network. ChangedSession=[%ls] Invitation=[%ls]", *SessionName.ToString(), *InvitationKey);

		bExpectingNewInvitationData = false;
		AuthenticateWithNetwork(InvitationKey);
	}
}

void FPlayFabPartySocket::HandleSessionParticipantJoined(FName SessionName, const FUniqueNetId& UniqueId)
{
	UE_LOGF(LogPlayFabParty, VeryVerbose, "Received SessionParticipantJoined event. ChangedSession=[%ls] UniqueId=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString());

	// Not for this session
	if (BoundSessionName != SessionName)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring SessionParticipantJoined event for wrong session. BoundSessionName=[%ls] ChangedSession=[%ls]", *BoundSessionName.ToString(), *SessionName.ToString());
		return;
	}

	// Only the host cares if players join/leave
	if (!bIsServer)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring SessionParticipantJoined event because we are a client. SessionName=[%ls]", *SessionName.ToString());
		return;
	}

	const uint64 Xuid = FCString::Strtoui64(*UniqueId.ToString(), nullptr, 10);
	if (Xuid == 0)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Received SessionParticipantJoined event for invalid participant. SessionName=[%ls] UniqueId=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString());
		return;
	}

	if (!SessionMembers.Contains(Xuid))
	{
		SessionMembers.Add(Xuid);

		UE_LOGF(LogPlayFabParty, Log, "Creating new session invitation as a new player has joined. SessionName=[%ls] NewUniqueId=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString());

		CreateInvitation();
	}
	else
	{
		UE_LOGF(LogPlayFabParty, Warning, "Received SessionParticipantJoined event for participant who is already registered in session. SessionName=[%ls] UniqueId=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString());
	}
}

void FPlayFabPartySocket::HandleSessionParticipantLeft(FName SessionName, const FUniqueNetId& UniqueId, EOnSessionParticipantLeftReason LeaveReason)
{
	UE_LOGF(LogPlayFabParty, VeryVerbose, "Received SessionParticipantLeft event. ChangedSession=[%ls] UniqueId=[%ls] Motive=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString(), ToLogString(LeaveReason));

	// Not for this session
	if (BoundSessionName != SessionName)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring SessionParticipantLeft event for wrong session. BoundSessionName=[%ls] ChangedSession=[%ls]", *BoundSessionName.ToString(), *SessionName.ToString());
		return;
	}

	// Only the host cares if players join/leave
	if (!bIsServer)
	{
		UE_LOGF(LogPlayFabParty, VeryVerbose, "Ignoring SessionParticipantLeft event because we are a client. SessionName=[%ls]", *SessionName.ToString());
		return;
	}

	const uint64 Xuid = FCString::Strtoui64(*UniqueId.ToString(), nullptr, 10);
	if (Xuid == 0)
	{
		UE_LOGF(LogPlayFabParty, Warning, "Received SessionParticipantLeft event for invalid participant. SessionName=[%ls] UniqueId=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString());
		return;
	}

	if (SessionMembers.Remove(Xuid) > 0)
	{
		UE_LOGF(LogPlayFabParty, Log, "Creating new session invitation as an existing player has left. SessionName=[%ls] NewUniqueId=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString());

		CreateInvitation();
	}
	else
	{
		UE_LOGF(LogPlayFabParty, Warning, "Received SessionParticipantLeft event for participant who was not registered in session. SessionName=[%ls] UniqueId=[%ls]", *SessionName.ToString(), *UniqueId.ToDebugString());
	}
}

void FPlayFabPartySocket::HandleAuthorizationFailure()
{
	UE_LOGF(LogPlayFabParty, Warning, "Attempt to authorization failed, it could be old invite. Will try to wait for new invitation data then retry.");
	bExpectingNewInvitationData = true;
	TimeToWaitForNewInvitationData = CVarPlayFabTimeToWaitForEvent.GetValueOnGameThread();
}

#endif // WITH_PLAYFAB_PARTY
