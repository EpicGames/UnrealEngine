// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_PLAYFAB_PARTY
#include "CoreMinimal.h"
#include "Sockets.h"
#include "Containers/Queue.h"
#include "PlayFabPartyInternetAddr.h"
#include "Interfaces/OnlineSessionInterface.h"

namespace Party
{
	class PartyInvitation;
	class PartyLocalEndpoint;
	class PartyLocalUser;
	class PartyNetwork;
}

class FPlayFabPartySocketSubsystem;
class FPlayFabPartySocket;
class IOnlineSubsystem;
class IOnlineSession;
class FOnlineSessionSettings;
class FUniqueNetId;

struct FPlayFabPartyPendingPacket
{
public:
	TArray<uint8> Data;
	FPlayFabPartyInternetAddr Sender;
};

class FPlayFabPartySocket
	: public FSocket
{
public:
	explicit FPlayFabPartySocket(FPlayFabPartySocketSubsystem& SocketSubsystem, IOnlineSubsystem& OnlineSubsystem, const FName SessionName);
	virtual ~FPlayFabPartySocket();

	//~ Begin FSocket Interface
	virtual bool Shutdown(ESocketShutdownMode Mode) override;
	virtual bool Close() override;
	virtual bool Bind(const FInternetAddr& Addr) override;
	virtual bool Connect(const FInternetAddr& Addr) override;
	virtual bool Listen(int32 MaxBacklog) override;
	virtual bool WaitForPendingConnection(bool& bHasPendingConnection, const FTimespan& WaitTime) override;
	virtual bool HasPendingData(uint32& PendingDataSize) override;
	virtual FSocket* Accept(const FString& InSocketDescription) override;
	virtual FSocket* Accept(FInternetAddr& OutAddr, const FString& InSocketDescription) override;
	virtual bool SendTo(const uint8* Data, int32 Count, int32& BytesSent, const FInternetAddr& Destination) override;
	virtual bool Send(const uint8* Data, int32 Count, int32& BytesSent) override;
	virtual bool RecvFrom(uint8* Data, int32 BufferSize, int32& BytesRead, FInternetAddr& Source, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;
	virtual bool Recv(uint8* Data, int32 BufferSize, int32& BytesRead, ESocketReceiveFlags::Type Flags = ESocketReceiveFlags::None) override;
	virtual bool Wait(ESocketWaitConditions::Type Condition, FTimespan WaitTime) override;
	virtual ESocketConnectionState GetConnectionState() override;
	virtual void GetAddress(FInternetAddr& OutAddr) override;
	virtual bool GetPeerAddress(FInternetAddr& OutAddr) override;
	virtual bool SetNonBlocking(bool bIsNonBlocking = true) override;
	virtual bool SetBroadcast(bool bAllowBroadcast = true) override;
	virtual bool SetNoDelay(bool bIsNoDelay = true) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool JoinMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress) override;
	virtual bool LeaveMulticastGroup(const FInternetAddr& GroupAddress, const FInternetAddr& InterfaceAddress) override;
	virtual bool SetMulticastLoopback(bool bLoopback) override;
	virtual bool SetMulticastTtl(uint8 TimeToLive) override;
	virtual bool SetMulticastInterface(const FInternetAddr& InterfaceAddress) override;
	virtual bool SetReuseAddr(bool bAllowReuse = true) override;
	virtual bool SetLinger(bool bShouldLinger = true, int32 Timeout = 0) override;
	virtual bool SetRecvErr(bool bUseErrorQueue = true) override;
	virtual bool SetSendBufferSize(int32 Size, int32& NewSize) override;
	virtual bool SetReceiveBufferSize(int32 Size, int32& NewSize) override;
	virtual int32 GetPortNo() override;
	//~ End FSocket Interface

	void TickDispatch(float DeltaTime);

	/** Get the socket context used for mapping PlayFab async events to this socket*/
	uint64 GetContext() const;

	/** True if we were started in listen mode, or false if we a client */
	bool IsServer() const;

	/** True if we are currently fully connected to a network, false otherwise */
	bool IsNetworkConnected() const;
	/** True if we have authenticated with the network, false otherwise */
	bool HasAuthenticated() const;
	/** True if we have successfully created our endpoint, false otherwise */
	bool IsEndpointConnected() const;

	/** Authenticate our local user with the network with the provided invitation id */
	bool AuthenticateWithNetwork(const FString& NetworkInvitation);
	/** Create our local endpoint so we can send and receive messages*/
	bool CreateLocalEndPoint();
	/** Attempt to create a new invitation for all members of our session */
	bool CreateInvitation();

	/** Handle updating state now that we've connected to the network */
	void HandleConnectToNetworkComplete();
	/** Handle updating state now that we've connected to the network */
	void HandleLocalUserAuthenticationComplete();
	/** Handle updating state now that we've created our local endpoint */
	void HandleCreateLocalEndpointComplete();
	/** Handle queuing a packet to be read */
	void HandleEndpointMessageReceived(FPlayFabPartyPendingPacket&& Message);
	/** Handle writing our new invitation to the session */
	void HandleCreateInvitationCompleted(Party::PartyInvitation* Invitation, const bool bWasSuccessful);
	/** Handle checking if we have entity ids for all of our players and creating our invitation if so */
	void HandleGetEntityIdsFromXboxLiveUserIdsCompleted(const bool bWasSuccessful);

	/** handle auth failure. If we have already retried once, close the socket. */
	void HandleAuthorizationFailure();

protected:
	Party::PartyLocalUser* GetLocalUser();

	bool SetNetworkAddress(const FInternetAddr& Addr);

	bool Connect_Internal();

	void HandleCreateSessionComplete(FName SessionName, bool bWasSuccessful);
	void HandleSessionCustomDataChanged(FName SessionName, const FOnlineSessionSettings& SessionSettings);
	void HandleSessionParticipantJoined(FName SessionName, const FUniqueNetId& UniqueId);
	void HandleSessionParticipantLeft(FName SessionName, const FUniqueNetId& UniqueId, EOnSessionParticipantLeftReason LeaveReason);

	bool CloseImpl();

protected:
	/** Reference to our subsystem */
	FPlayFabPartySocketSubsystem& SocketSubsystem;
	/** Weak reference to the online subsystem we're mapped to's session interface */
	TWeakPtr<IOnlineSession, ESPMode::ThreadSafe> WeakSessionInterface;
	/** What SessionName should we be using from the OSS? */
	FName BoundSessionName;
	/** Unique ID to refer to this socket by */
	uint64 SocketContext = 0;

	/** Handle to our OSS CreateSession binding */
	FDelegateHandle CreateSessionCompleteHandle;
	/** Handle to our OSS SessionCustomDataChanged binding */
	FDelegateHandle SessionCustomDataChangedHandle;
	/** Handle to our OSS SessionParticipantJoined binding */
	FDelegateHandle SessionParticipantJoinedHandle;
	/** Handle to our OSS SessionParticipantLeft binding */
	FDelegateHandle SessionParticipantLeftHandle;

	/** Address of network be belong to (if one is set) */
	TSharedPtr<FPlayFabPartyInternetAddr> NetworkAddress;

	/** Are we the server? */
	bool bIsServer = false;
	/** The current party invitation (only set on server) */
	Party::PartyInvitation* PartyInvitation = nullptr;
	/** The XUIDs of the members of our party (except host) (only set on server) */
	TArray<uint64> SessionMembers;

	/** Network instance of the network we belong to (if we are connected to one) */
    Party::PartyNetwork* PartyNetwork = nullptr;
	/** Have we successfully connected to the network? */
	bool bIsNetworkConnected = false;

	/** String to use to authenticate with the network when we are fully connected */
	TOptional<FString> PendingNetworkInvitation;
	/** Have we successfully authenticated with the network? */
	bool bIsLocalUserAuthenticated = false;

	/** Our local endpoint of the network we belong to (if we are connected to one) */
	Party::PartyLocalEndpoint* PartyLocalEndpoint = nullptr;
	/** Has our endpoint successfully joined the network? */
	bool bIsEndpointConnected = false;

	/** If the previous socket still leaving playfab network, mark this as true, then connect when leave network complete */
	bool bExpectingLeaveNetworkComplete = false;
	/** If leave network complete event never get notified, timeout after this duration */
	float TimeToWaitForLeaveNetworkComplete = 0.0f;

	TOptional<ESocketErrors> SocketConnectError;

	/** Received packets that have not been processed yet */
	TQueue<TUniquePtr<FPlayFabPartyPendingPacket>, EQueueMode::Spsc> PacketQueue;

	/** If failed to authenticate local user in client, it could be client hasn't get the updated session invitation data, will retry it after receiving it*/
	bool bExpectingNewInvitationData = false;
	/** If custom session data changed event never get notified, timeout after this duration */
	float TimeToWaitForNewInvitationData;
};
#endif // WITH_PLAYFAB_PARTY
