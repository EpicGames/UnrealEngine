// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_PLAYFAB_PARTY
#include "PlayFabPartySocketSubsystem.h"
#include "PlayFabPartyInternetAddr.h"
#include "PlayFabPartySocket.h"
#include "PlayFabPartyManager.h"
#include "Misc/CoreDelegates.h"

FPlayFabPartySocketSubsystem::FPlayFabPartySocketSubsystem()
	: SocketContextGenerator(0)
{
}

FPlayFabPartySocketSubsystem::~FPlayFabPartySocketSubsystem()
{
}

bool FPlayFabPartySocketSubsystem::Init(FString& Error)
{
	if (PlayFabPartyManager.IsValid())
	{
		return true;
	}

	TOptional<FString> AppId = FPlayFabPartyManager::GetAppId();
	if (!AppId.IsSet())
	{
		Error = TEXT("PlayFab AppId either unset or invalid, cannot initialize PlayFabParty SocketSubsystem");
		return false;
	}

	PlayFabPartyManager = FPlayFabPartyManager::CreateManager(*this, AppId.GetValue(), Error);
	if (!PlayFabPartyManager.IsValid())
	{
		// CreateManager will set Error in failure cases
		return false;
	}
	
	// destroy party manager as soon as the engine exits because XblParty shutdown code depends on XGameRuntime
	FCoreDelegates::OnPreExit.AddLambda( [this]()
	{
		PlayFabPartyManager.Reset();
	});

	return true;
}

void FPlayFabPartySocketSubsystem::Shutdown()
{
	PlayFabPartyManager.Reset();
}

FSocket* FPlayFabPartySocketSubsystem::CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolName)
{
	return nullptr;
}

void FPlayFabPartySocketSubsystem::DestroySocket(FSocket* Socket)
{
	delete Socket;
}

FAddressInfoResult FPlayFabPartySocketSubsystem::GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName, EAddressInfoFlags QueryFlags, const FName ProtocolTypeName, ESocketType SocketType)
{
	return FAddressInfoResult(nullptr, nullptr);
}

TSharedPtr<FInternetAddr> FPlayFabPartySocketSubsystem::GetAddressFromString(const FString& InAddress)
{
	TSharedPtr<FPlayFabPartyInternetAddr> InternetAddr = MakeShared<FPlayFabPartyInternetAddr>(InAddress);
	if (!InternetAddr->IsValid())
	{
		return nullptr;
	}

	return InternetAddr;
}

bool FPlayFabPartySocketSubsystem::RequiresChatDataBeSeparate()
{
	return false;
}

bool FPlayFabPartySocketSubsystem::RequiresEncryptedPackets()
{
	return false;
}

bool FPlayFabPartySocketSubsystem::GetHostName(FString& HostName)
{
	return false;
}

TSharedRef<FInternetAddr> FPlayFabPartySocketSubsystem::CreateInternetAddr()
{
	return MakeShared<FPlayFabPartyInternetAddr>();
}

bool FPlayFabPartySocketSubsystem::HasNetworkDevice()
{
	return false;
}

const TCHAR* FPlayFabPartySocketSubsystem::GetSocketAPIName() const
{
	return PLAYFABPARTY_SOCKETSUBSYSTEM;
}

ESocketErrors FPlayFabPartySocketSubsystem::GetLastErrorCode()
{
	return LastSocketError;
}

ESocketErrors FPlayFabPartySocketSubsystem::TranslateErrorCode(int32 Code)
{
	return static_cast<ESocketErrors>(Code);
}

bool FPlayFabPartySocketSubsystem::IsSocketWaitSupported() const
{
	return false;
}

bool FPlayFabPartySocketSubsystem::IsPlayFabPartyInitialized() const
{
	return PlayFabPartyManager.IsValid() && PlayFabPartyManager->IsInitialized();
}

bool FPlayFabPartySocketSubsystem::IsPlayFabPartyReady() const
{
	return IsPlayFabPartyInitialized() && PlayFabPartyManager->IsReady();
}

TSharedPtr<FSocket> FPlayFabPartySocketSubsystem::CreatePlayFabSocket(IOnlineSubsystem& OnlineSubsystem, const FName SessionName, bool bReuseSocket)
{
	if ( bReuseSocket )
	{
		if (PlayFabPartyManager->GetPlayFabSocketListenServer())
		{
			return PlayFabPartyManager->GetPlayFabSocketListenServer();
		}

		TSharedPtr<FSocket> Socket(new FPlayFabPartySocket(*this, OnlineSubsystem, SessionName));
		PlayFabPartyManager->SetPlayFabSocketListenServer(Socket);
		return Socket;
	}

	return TSharedPtr<FSocket>(new FPlayFabPartySocket(*this, OnlineSubsystem, SessionName));
}

uint64 FPlayFabPartySocketSubsystem::GenerateSocketContext(FPlayFabPartySocket* Socket)
{
	uint64 Context = ++SocketContextGenerator;
	ContextToSocketMap.Add(Context, Socket);
	return Context;
}

FPlayFabPartySocket* FPlayFabPartySocketSubsystem::GetSocketFromContext(const uint64 Context)
{
	if (FPlayFabPartySocket** SocketPtr = ContextToSocketMap.Find(Context))
	{
		return *SocketPtr;
	}

	return nullptr;
}

void FPlayFabPartySocketSubsystem::ReleaseSocketContext(const uint64 Context)
{
	ContextToSocketMap.Remove(Context);
}

TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> FPlayFabPartySocketSubsystem::GetPlayFabPartyManager() const
{
	if (!IsPlayFabPartyReady())
	{
		return nullptr;
	}

	return PlayFabPartyManager;
}

void FPlayFabPartySocketSubsystem::SetLastSocketError(const ESocketErrors NewSocketError)
{
	LastSocketError = NewSocketError;
}
#endif // WITH_PLAYFAB_PARTY
