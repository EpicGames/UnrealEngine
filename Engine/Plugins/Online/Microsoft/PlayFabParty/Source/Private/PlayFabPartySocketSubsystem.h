// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_PLAYFAB_PARTY

#include "CoreMinimal.h"
#include "SocketSubsystem.h"
#include "PlayFabPartyManager.h"
#include "Containers/SortedMap.h"

#define PLAYFABPARTY_SOCKETSUBSYSTEM TEXT("PlayFabParty")

class FPlayFabPartySocket;
class IOnlineSubsystem;

class FPlayFabPartySocketSubsystem : public ISocketSubsystem
{
public:
	FPlayFabPartySocketSubsystem();
	virtual ~FPlayFabPartySocketSubsystem();

	//~ Begin ISocketSubsystem Interface
	virtual bool Init(FString& Error) override;
	virtual void Shutdown() override;
	virtual FSocket* CreateSocket(const FName& SocketType, const FString& SocketDescription, const FName& ProtocolName) override;
	virtual void DestroySocket(FSocket* Socket) override;
	virtual FAddressInfoResult GetAddressInfo(const TCHAR* HostName, const TCHAR* ServiceName = nullptr, EAddressInfoFlags QueryFlags = EAddressInfoFlags::Default, const FName ProtocolTypeName = NAME_None, ESocketType SocketType = ESocketType::SOCKTYPE_Unknown) override;
	virtual TSharedPtr<FInternetAddr> GetAddressFromString(const FString& InAddress) override;
	virtual bool RequiresChatDataBeSeparate() override;
	virtual bool RequiresEncryptedPackets() override;
	virtual bool GetHostName(FString& HostName) override;
	virtual TSharedRef<FInternetAddr> CreateInternetAddr() override;
	virtual bool HasNetworkDevice() override;
	virtual const TCHAR* GetSocketAPIName() const override;
	virtual ESocketErrors GetLastErrorCode() override;
	virtual ESocketErrors TranslateErrorCode(int32 Code) override;
	virtual bool IsSocketWaitSupported() const override;
	//~ End ISocketSubsystem Interface

	/** Is the PlayFabParty initialized */
	bool IsPlayFabPartyInitialized() const;
	/** Is the PlayFabParty ready for use */
	bool IsPlayFabPartyReady() const;

	/** Create a PlayFabParty socket */
	TSharedPtr<FSocket> CreatePlayFabSocket(IOnlineSubsystem& OnlineSubsystem, const FName SessionName, bool bReuseSocket);

	/** Generate a new socket context and map it to the provided socket */
	uint64 GenerateSocketContext(FPlayFabPartySocket* Socket);
	/** Get the socket that maps to a provided context */
	FPlayFabPartySocket* GetSocketFromContext(const uint64 Context);
	/** Release the mapping of the provided context to the socket it was mapped to */
	void ReleaseSocketContext(const uint64 Context);

	friend FPlayFabPartySocket;
	/** Get the PlayFabParty manager (only available if it is ready) */
	TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> GetPlayFabPartyManager() const;

protected:
	friend class UPlayFabPartyNetDriver;
	/** Set the last socket error that occurred */
	void SetLastSocketError(const ESocketErrors NewSocketError);

protected:
	/** Generator for creating contexts for sockets */
	TAtomic<uint64> SocketContextGenerator;

	/** Our PlayFabParty Manager Object Instance */
	TSharedPtr<FPlayFabPartyManager, ESPMode::ThreadSafe> PlayFabPartyManager;

	/** The last error we received */
	ESocketErrors LastSocketError = ESocketErrors::SE_NO_ERROR;

	/** Map to translate socket contexts into sockets from callbacks */
	TSortedMap<uint64, FPlayFabPartySocket*> ContextToSocketMap;
};

#endif // WITH_PLAYFAB_PARTY
