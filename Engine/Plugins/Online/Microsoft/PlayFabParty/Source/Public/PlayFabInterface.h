// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

/** Do we have a cached Entity ID for the provided Xuid? */
PLAYFABPARTY_API bool HaveEntityIdForXuid(const uint64 Xuid);
/** Get the cached Entity ID for the provided Xuid. Asserts if we do not have a value cached! */
PLAYFABPARTY_API const char* GetEntityIdForXuid(const uint64 Xuid);
/** Get the cached Entity Token for the provided Xuid if any. */
PLAYFABPARTY_API const char* GetEntityTokenForXuid(const uint64 Xuid);
